{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Batch Renamer",
	"Description": "Rename multiple selected actors or assets, and standardize their prefixes and suffixes.",
	"Category": "Experimental",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"EnabledByDefault": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "AdvancedRenamer",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}
