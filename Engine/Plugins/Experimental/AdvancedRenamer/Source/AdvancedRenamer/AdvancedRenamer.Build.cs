// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AdvancedRenamer : ModuleRules
{
	public AdvancedRenamer(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ContentBrowser",
				"CoreUObject",
				"Engine",
				"InputCore",
				"LevelEditor",
				"Projects",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"UMG",
				"UnrealEd"
			}
		);
	}
}
