// Copyright Epic Games, Inc. All Rights Reserved.

#include "AdvancedRenamer.h"

#include "AdvancedRenamerModule.h"

FAdvancedRenamer::FAdvancedRenamer(const TSharedRef<IAdvancedRenamerProvider>& InProvider)
	: Provider(InProvider)
{
	int32 RenamedEntitiesCount = Num();
	check(RenamedEntitiesCount > 0);

	for (int32 Index = 0; Index < RenamedEntitiesCount; ++Index)
	{
		if (!CanRename(Index))
		{
			RemoveIndex(Index);
			--Index;
			--RenamedEntitiesCount;
			continue;
		}

		const int32 Hash = GetHash(Index);
		FString OriginalName = GetOriginalName(Index);

		Previews.Add(MakeShared<FAdvancedRenamerPreview>(Hash, OriginalName));
	}
	SortablePreviews = Previews;
}

const TSharedRef<IAdvancedRenamerProvider>& FAdvancedRenamer::GetProvider() const
{
	return Provider;
}

TArray<TSharedPtr<FAdvancedRenamerPreview>>& FAdvancedRenamer::GetSortablePreviews()
{
	return SortablePreviews;
}

void FAdvancedRenamer::ResetSortablePreviews()
{
	SortablePreviews = Previews;
}

void FAdvancedRenamer::AddSection(FAdvancedRenamerExecuteSection InSection)
{
	Sections.Add(InSection);
}

bool FAdvancedRenamer::HasRenames() const
{
	return bHasRenames;
}

bool FAdvancedRenamer::IsDirty() const
{
	return bDirty;
}

void FAdvancedRenamer::MarkDirty()
{
	bDirty = true;
}

void FAdvancedRenamer::MarkClean()
{
	bDirty = false;
}

bool FAdvancedRenamer::UpdatePreviews()
{
	bHasRenames = false;
	const int32 Count = SortablePreviews.Num();

	BeforeOperationsStartExecute();

	for (int32 Index = 0; Index < Count; ++Index)
	{
		if (!SortablePreviews[Index].IsValid() || !IsValidIndex(Index))
		{
			RemoveIndex(Index);
			--Index;
			continue;
		}

		// Force recreation
		SortablePreviews[Index]->NewName = ApplyRename(SortablePreviews[Index]->OriginalName);

		if (SortablePreviews[Index]->NewName.IsEmpty())
		{
			continue;
		}

		if (GetOriginalName(Index).Equals(SortablePreviews[Index]->NewName))
		{
			continue;
		}

		bHasRenames = true;
	}

	AfterOperationsEndExecute();

	MarkClean();

	return bHasRenames;
}

bool FAdvancedRenamer::Execute()
{
	if (!HasRenames())
	{
		UpdatePreviews();

		if (!HasRenames())
		{
			return false;
		}
	}

	const int32 Count = SortablePreviews.Num();

	bool bAllSuccess = Provider->BeginRename();
	for (int32 Index = 0; Index < Count; ++Index)
	{
		if (!SortablePreviews[Index].IsValid()
			|| !IsValidIndex(Index)
			|| SortablePreviews[Index]->NewName.IsEmpty())
		{
			continue;
		}

		bAllSuccess &= Provider->PrepareRename(Index, SortablePreviews[Index]->NewName);
	}
	bAllSuccess &= Provider->ExecuteRename();
	bAllSuccess &= Provider->EndRename();

	MarkClean();

	return bAllSuccess;
}

FString FAdvancedRenamer::ApplyRename(const FString& InOriginalName)
{
	FString NewName = InOriginalName;
	for (FAdvancedRenamerExecuteSection& Section : Sections)
	{
		Section.OnOperationExecuted().ExecuteIfBound(NewName);
	}
	return NewName;
}

void FAdvancedRenamer::BeforeOperationsStartExecute()
{
	for (FAdvancedRenamerExecuteSection& Section : Sections)
	{
		Section.OnBeforeOperationExecutionStart().ExecuteIfBound();
	}
}

void FAdvancedRenamer::AfterOperationsEndExecute()
{
	for (FAdvancedRenamerExecuteSection& Section : Sections)
	{
		Section.OnAfterOperationExecutionEnded().ExecuteIfBound();
	}
}

int32 FAdvancedRenamer::Num() const
{
	return Provider->Num();
}

bool FAdvancedRenamer::IsValidIndex(int32 InIndex) const
{
	return Provider->IsValidIndex(InIndex);
}

uint32 FAdvancedRenamer::GetHash(int32 InIndex) const
{
	return Provider->GetHash(InIndex);
}

FString FAdvancedRenamer::GetOriginalName(int32 InIndex) const
{
	return Provider->GetOriginalName(InIndex);
}

bool FAdvancedRenamer::RemoveIndex(int32 InIndex)
{
	// Can fail during construction when indices that aren't renameable are removed from the provider before
	// they are added to ListData.
	int32 OriginalIndex = INDEX_NONE;
	if (SortablePreviews.IsValidIndex(InIndex))
	{
		OriginalIndex = Previews.IndexOfByKey(SortablePreviews[InIndex]);
		SortablePreviews.RemoveAt(InIndex);
	}

	return Provider->RemoveIndex(OriginalIndex);
}

bool FAdvancedRenamer::CanRename(int32 InIndex) const
{
	return Provider->CanRename(InIndex);
}

bool FAdvancedRenamer::BeginRename()
{
	return Provider->BeginRename();
}

bool FAdvancedRenamer::PrepareRename(int32 InIndex, const FString& InNewName)
{
	return Provider->PrepareRename(InIndex, InNewName);
}

bool FAdvancedRenamer::ExecuteRename()
{
	return Provider->ExecuteRename();
}

bool FAdvancedRenamer::EndRename()
{
	return Provider->EndRename();
}