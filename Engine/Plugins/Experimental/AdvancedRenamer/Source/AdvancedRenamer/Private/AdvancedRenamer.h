// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAdvancedRenamer.h"
#include "Providers/IAdvancedRenamerProvider.h"
#include "Templates/SharedPointer.h"

struct FAdvancedRenamerPreview;

class FAdvancedRenamer : public IAdvancedRenamer
{
public:
	FAdvancedRenamer(const TSharedRef<IAdvancedRenamerProvider>& InProvider);

	//~ Begin IAdvancedRenamer
	virtual const TSharedRef<IAdvancedRenamerProvider>& GetProvider() const override;
	virtual TArray<TSharedPtr<FAdvancedRenamerPreview>>& GetSortablePreviews() override;
	virtual void ResetSortablePreviews() override;
	virtual void AddSection(FAdvancedRenamerExecuteSection InSection) override;
	virtual bool HasRenames() const override;
	virtual bool IsDirty() const override;
	virtual void MarkDirty() override;
	virtual void MarkClean() override;
	virtual FString ApplyRename(const FString& InName) override;
	virtual bool UpdatePreviews() override;
	virtual bool Execute() override;
	//~ End IAdvancedRenamer

	//~ Begin IAdvancedRenamerProvider
	virtual int32 Num() const override;
	virtual bool IsValidIndex(int32 InIndex) const override;
	virtual uint32 GetHash(int32 InIndex) const override;
	virtual FString GetOriginalName(int32 InIndex) const override;
	virtual bool RemoveIndex(int32 InIndex) override;
	virtual bool CanRename(int32 InIndex) const override;

	virtual bool BeginRename() override;
	virtual bool PrepareRename(int32 InIndex, const FString& InNewName) override;
	virtual bool ExecuteRename() override;
	virtual bool EndRename() override;
	//~ End IAdvancedRenamerProvider

private:
	/** Called before the whole Rename logic start */
	void BeforeOperationsStartExecute();
	
	/** Called after the whole Rename logic end */
	void AfterOperationsEndExecute();

private:
	/** Provider for this Renamer */
	TSharedRef<IAdvancedRenamerProvider> Provider;

	/** Previews Name list */
	TArray<TSharedPtr<FAdvancedRenamerPreview>> Previews;

	/** Sorted Previews Name list */
	TArray<TSharedPtr<FAdvancedRenamerPreview>> SortablePreviews;

	/** Renamer sections list */
	TArray<FAdvancedRenamerExecuteSection> Sections;

	/** Whether or not at least 1 preview has a Rename */
	bool bHasRenames;
	
	/** if true the Rename logic will be executed */
	bool bDirty;
};
