// Copyright Epic Games, Inc. All Rights Reserved.

#include "AdvancedRenamerCommands.h"
#include "AdvancedRenamerStyle.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Styling/ISlateStyle.h"

#define LOCTEXT_NAMESPACE "AdvancedRenamerCommands"

FAdvancedRenamerCommands::FAdvancedRenamerCommands()
	: TCommands<FAdvancedRenamerCommands>(
		TEXT("AdvancedRenamer"),
		LOCTEXT("AdvancedRenamer", "Advanced Renamer"),
		NAME_None,
		FAdvancedRenamerStyle::Get().GetStyleSetName()
	)
{
}

void FAdvancedRenamerCommands::RegisterCommands()
{
	UI_COMMAND(BatchRenameObject, "Batch Rename", "Batch Rename Object(s) based on selection.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::F2));
	UI_COMMAND(BatchRenameSharedClassActors, "Rename Actors of Selected Actor Classes", "Opens the Batch Renamer Panel to rename all actors sharing a class with any selected actor.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::F2));
}

#undef LOCTEXT_NAMESPACE
