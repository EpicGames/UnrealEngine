// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Templates/SharedPointer.h"

class FUICommandInfo;

class FAdvancedRenamerCommands : public TCommands<FAdvancedRenamerCommands>
{
public:
	FAdvancedRenamerCommands();

	virtual void RegisterCommands() override;

	TSharedPtr<FUICommandInfo> BatchRenameObject;
	TSharedPtr<FUICommandInfo> BatchRenameSharedClassActors;
};
