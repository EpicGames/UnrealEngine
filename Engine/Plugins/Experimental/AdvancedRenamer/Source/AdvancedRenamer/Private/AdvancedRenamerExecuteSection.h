// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "UObject/NameTypes.h"

DECLARE_DELEGATE(FOnBeforeOperationExecutionStart)
DECLARE_DELEGATE_OneParam(FOnOperationExecuted, FString& /** OutString */)
DECLARE_DELEGATE(FOnAfterOperationExecutionEnded)

struct FAdvancedRenamerExecuteSection
{
	/** Get the callback executed before the whole Rename happens */
	FOnOperationExecuted& OnOperationExecuted() { return OnOperationExecutedDelegate; }

	/** Get the callback executed during the Rename */
	FOnBeforeOperationExecutionStart& OnBeforeOperationExecutionStart() { return OnBeforeOperationExecutionStartDelegate; }

	/** Get the callback executed after the whole Rename happens */
	FOnAfterOperationExecutionEnded& OnAfterOperationExecutionEnded() { return OnAfterOperationExecutionEndedDelegate; }

public:
	/** Name of this section */
	FName SectionName;

private:
	/** Callback called before the whole Rename execution is applied */
	FOnBeforeOperationExecutionStart OnBeforeOperationExecutionStartDelegate;

	/** Callback called during the whole Rename execution */
	FOnOperationExecuted OnOperationExecutedDelegate;

	/** Callback called after the whole Rename execution is applied */
	FOnAfterOperationExecutionEnded OnAfterOperationExecutionEndedDelegate;
};
