// Copyright Epic Games, Inc. All Rights Reserved.

#include "AdvancedRenamerModule.h"

#include "AdvancedRenamer.h"
#include "AdvancedRenamerCommands.h"
#include "AdvancedRenamerSections/AdvancedRenamerAddPrefixSuffixSection.h"
#include "AdvancedRenamerSections/AdvancedRenamerChangeCaseSection.h"
#include "AdvancedRenamerSections/AdvancedRenamerNumberingSection.h"
#include "AdvancedRenamerSections/AdvancedRenamerRemovePrefixSection.h"
#include "AdvancedRenamerSections/AdvancedRenamerRemoveSuffixSection.h"
#include "AdvancedRenamerSections/AdvancedRenamerSearchAndReplaceSection.h"
#include "AdvancedRenamerStyle.h"
#include "EngineUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "Integrations/AdvancedRenamerContentBrowserIntegration.h"
#include "Integrations/AdvancedRenamerLevelEditorIntegration.h"
#include "Providers/AdvancedRenamerActorProvider.h"
#include "Slate/SAdvancedRenamerPanel.h"
#include "Toolkits/IToolkitHost.h"
#include "Widgets/SWindow.h"

DEFINE_LOG_CATEGORY(LogARP);

#define LOCTEXT_NAMESPACE "AdvancedRenamerModule"

namespace UE::AdvancedRenamer::Private
{
	TSharedRef<SWindow> CreateAdvancedRenamerWindow()
	{
		// Workaround to make the AppScale and OS Zoom work at the same time, this will eventually be changed to add constraint to the window size
		const float AppScale = FSlateApplication::Get().GetApplicationScale();
		constexpr float MinWindowHeight = 589.f;
		constexpr float MinWindowWidth = 730.f;
		constexpr float TitleHeightOffset = 38.f;
		constexpr float ContentWidthOffset = 6.f;
		return SNew(SWindow)
			.Title(LOCTEXT("AdvancedRenameWindow", "Batch Renamer"))
			.ClientSize(FVector2D((MinWindowWidth + ContentWidthOffset) * AppScale, (MinWindowHeight + TitleHeightOffset) * AppScale))
			.MinHeight((MinWindowHeight + TitleHeightOffset) * AppScale)
			.MinWidth((MinWindowWidth + ContentWidthOffset) * AppScale);
	}
}

void FAdvancedRenamerModule::StartupModule()
{
	FAdvancedRenamerStyle::Initialize();
	FAdvancedRenamerCommands::Register();
	FAdvancedRenamerContentBrowserIntegration::Initialize();
	FAdvancedRenamerLevelEditorIntegration::Initialize();
	RegisterDefaultSections();
}

void FAdvancedRenamerModule::ShutdownModule()
{
	FAdvancedRenamerStyle::Shutdown();
	FAdvancedRenamerCommands::Unregister();
	FAdvancedRenamerContentBrowserIntegration::Shutdown();
	FAdvancedRenamerLevelEditorIntegration::Shutdown();
}

TSharedRef<IAdvancedRenamer> FAdvancedRenamerModule::CreateAdvancedRenamer(const TSharedRef<IAdvancedRenamerProvider>& InRenameProvider)
{
	return MakeShared<FAdvancedRenamer>(InRenameProvider);
}

void FAdvancedRenamerModule::OpenAdvancedRenamer(const TSharedRef<IAdvancedRenamerProvider>& InRenameProvider, const TSharedPtr<SWidget>& InParentWidget)
{
	return OpenAdvancedRenamer(CreateAdvancedRenamer(InRenameProvider), InParentWidget);
}

void FAdvancedRenamerModule::OpenAdvancedRenamer(const TSharedRef<IAdvancedRenamerProvider>& InRenameProvider, const TSharedPtr<IToolkitHost>& InToolkitHost)
{
	return OpenAdvancedRenamer(CreateAdvancedRenamer(InRenameProvider), InToolkitHost);
}

void FAdvancedRenamerModule::OpenAdvancedRenamerForActors(const TArray<AActor*>& InActors, const TSharedPtr<SWidget>& InParentWidget)
{
	TArray<TWeakObjectPtr<AActor>> WeakObjects;

	WeakObjects.Reserve(InActors.Num());

	Algo::Transform(
		InActors,
		WeakObjects,
		[](AActor* InActor)
		{
			return TWeakObjectPtr<AActor>(InActor);
		}
	);
	
	OnFilterAdvancedRenamerActors().Broadcast(WeakObjects);

	// Don't open the advanced renamer if there are no actors remaining after the filter
	if (WeakObjects.IsEmpty())
	{
		return;
	}

	TSharedRef<FAdvancedRenamerActorProvider> ActorProvider = MakeShared<FAdvancedRenamerActorProvider>();
	ActorProvider->SetActorList(WeakObjects);

	OpenAdvancedRenamer(ActorProvider, InParentWidget);
}

void FAdvancedRenamerModule::OpenAdvancedRenamerForActors(const TArray<AActor*>& InActors, const TSharedPtr<IToolkitHost>& InToolkitHost)
{
	if (InToolkitHost.IsValid())
	{
		OpenAdvancedRenamerForActors(InActors, InToolkitHost->GetParentWidget());
	}
}

void FAdvancedRenamerModule::OpenAdvancedRenamer(const TSharedRef<IAdvancedRenamer>& InRenamer, const TSharedPtr<IToolkitHost>& InToolkitHost)
{
	if (InToolkitHost.IsValid())
	{
		OpenAdvancedRenamer(InRenamer, InToolkitHost->GetParentWidget());
	}
}

void FAdvancedRenamerModule::OpenAdvancedRenamer(const TSharedRef<IAdvancedRenamer>& InRenamer, const TSharedPtr<SWidget>& InParentWidget)
{
	TSharedRef<SWindow> AdvancedRenameWindow = UE::AdvancedRenamer::Private::CreateAdvancedRenamerWindow();
	AdvancedRenameWindow->SetContent(SNew(SAdvancedRenamerPanel, InRenamer));

	TSharedPtr<SWidget> ParentWindow = FSlateApplication::Get().FindBestParentWindowForDialogs(InParentWidget);
	FSlateApplication::Get().AddModalWindow(AdvancedRenameWindow, ParentWindow);
}

TArray<AActor*> FAdvancedRenamerModule::GetActorsSharingClassesInWorld(const TArray<AActor*>& InActors)
{
	TSet<UClass*> SelectedClasses;
	bool bHasActorClass = false;
	UWorld* World = nullptr;

	// Scan selected items and add valid classes to the selected classes list.
	for (AActor* SelectedActor : InActors)
	{
		if (!IsValid(SelectedActor))
		{
			continue;
		}

		if (!World)
		{
			World = SelectedActor->GetWorld();

			if (!World)
			{
				break;
			}
		}

		UClass* ActorClass = SelectedActor->GetClass();

		/**
		 * If we have a default AActor selected then all actors in the world share a
		 * class with the selected actors. We don't need anything other than the AActor
		 * class to get matches. Empty the array, store that and move on.
		 */
		if (ActorClass == AActor::StaticClass())
		{
			bHasActorClass = true;
			SelectedClasses.Empty();
			break;
		}

		SelectedClasses.Add(ActorClass);
	}

	if (!World)
	{
		return InActors;
	}

	TArray<UClass*> NonInheritingActorClasses;

	if (bHasActorClass)
	{
		NonInheritingActorClasses.Add(AActor::StaticClass());
	}
	else
	{
		for (UClass* ActorClass : SelectedClasses)
		{
			bool bFoundParent = false;

			for (UClass* ActorClassCheck : SelectedClasses)
			{
				if (ActorClass == ActorClassCheck)
				{
					continue;
				}

				if (ActorClass->IsChildOf(ActorClassCheck))
				{
					bFoundParent = true;
					break;
				}
			}

			if (!bFoundParent)
			{
				NonInheritingActorClasses.Add(ActorClass);
			}
		}
	}

	// Create outliner items for all the items matching the class list that are renameable.
	TArray<AActor*> AllActors;
	AllActors.Reserve(InActors.Num());

	for (UClass* ActorClass : NonInheritingActorClasses)
	{
		for (AActor* Actor : TActorRange<AActor>(World, ActorClass))
		{
			AllActors.Add(Actor);
		}
	}

	return AllActors;
}

void FAdvancedRenamerModule::RegisterDefaultSections()
{
	Sections.Add(IAdvancedRenamerSection::MakeInstance<FAdvancedRenamerSearchAndReplaceSection>());
	Sections.Add(IAdvancedRenamerSection::MakeInstance<FAdvancedRenamerRemovePrefixSection>());
	Sections.Add(IAdvancedRenamerSection::MakeInstance<FAdvancedRenamerRemoveSuffixSection>());
	Sections.Add(IAdvancedRenamerSection::MakeInstance<FAdvancedRenamerAddPrefixSuffixSection>());
	Sections.Add(IAdvancedRenamerSection::MakeInstance<FAdvancedRenamerNumberingSection>());
	Sections.Add(IAdvancedRenamerSection::MakeInstance<FAdvancedRenamerChangeCaseSection>());
}

IMPLEMENT_MODULE(FAdvancedRenamerModule, AdvancedRenamer)

#undef LOCTEXT_NAMESPACE
