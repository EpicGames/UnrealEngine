// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAdvancedRenamerModule.h"
#include "AdvancedRenamerSections/IAdvancedRenamerSection.h"
#include "Delegates/IDelegateInstance.h"
#include "Logging/LogMacros.h"

DECLARE_LOG_CATEGORY_EXTERN(LogARP, Log, All);

/** Advanced Rename Panel Plugin - Easily bulk rename stuff! */
class FAdvancedRenamerModule : public IAdvancedRenamerModule
{
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

	//~ Begin IAdvancedRenamerModule
	virtual TSharedRef<IAdvancedRenamer> CreateAdvancedRenamer(const TSharedRef<IAdvancedRenamerProvider>& InRenameProvider) override;
	virtual void OpenAdvancedRenamer(const TSharedRef<IAdvancedRenamerProvider>& InRenameProvider, const TSharedPtr<SWidget>& InParentWidget) override;
	virtual void OpenAdvancedRenamer(const TSharedRef<IAdvancedRenamerProvider>& InRenameProvider, const TSharedPtr<IToolkitHost>& InToolkitHost) override;
	virtual void OpenAdvancedRenamerForActors(const TArray<AActor*>& InActors, const TSharedPtr<SWidget>& InParentWidget) override;
	virtual void OpenAdvancedRenamerForActors(const TArray<AActor*>& InActors, const TSharedPtr<IToolkitHost>& InToolkitHost) override;
	virtual void OpenAdvancedRenamer(const TSharedRef<IAdvancedRenamer>& InRenamer, const TSharedPtr<IToolkitHost>& InToolkitHost) override;
	virtual void OpenAdvancedRenamer(const TSharedRef<IAdvancedRenamer>& InRenamer, const TSharedPtr<SWidget>& InParentWidget) override;
	virtual TArray<AActor*> GetActorsSharingClassesInWorld(const TArray<AActor*>& InActors) override;
	virtual TArray<TSharedPtr<IAdvancedRenamerSection>> GetRegisteredSections() const override { return Sections; }
	virtual FOnFilterAdvancedRenamerActors& OnFilterAdvancedRenamerActors() override { return FilterAdvancedRenamerActors; }
	//~ End IAdvancedRenamerModule

private:
	void RegisterDefaultSections();

private:
	TArray<TSharedPtr<IAdvancedRenamerSection>> Sections;
	FDelegateHandle EnableRenamerHandle;
	FOnFilterAdvancedRenamerActors FilterAdvancedRenamerActors;
};
