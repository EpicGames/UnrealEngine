// Copyright Epic Games, Inc. All Rights Reserved.

#include "AdvancedRenamerAddPrefixSuffixSection.h"

#include "AdvancedRenamerStyle.h"
#include "IAdvancedRenamer.h"
#include "Styling/StyleColors.h"
#include "Utils/AdvancedRenamerSlateUtils.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"

#define LOCTEXT_NAMESPACE "AdvancedRenamerAddPrefixSuffixSection"

FAdvancedRenamerAddPrefixSuffixSection::FAdvancedRenamerAddPrefixSuffixSection()
{
	FAdvancedRenamerAddPrefixSuffixSection::ResetToDefault();
}

void FAdvancedRenamerAddPrefixSuffixSection::Init(TSharedRef<IAdvancedRenamer> InRenamer)
{
	FAdvancedRenamerSectionBase::Init(InRenamer);
	Section.SectionName = TEXT("AddPrefixSuffixNumber");
	Section.OnOperationExecuted().BindSP(this, &FAdvancedRenamerAddPrefixSuffixSection::ApplyAddPrefixSuffixNumberOperation);
	InRenamer->AddSection(Section);
}

TSharedRef<SWidget> FAdvancedRenamerAddPrefixSuffixSection::GetWidget()
{
	using namespace AdvancedRenamerSlateUtils::Default;

	return SNew(SBorder)
		.BorderImage(FAdvancedRenamerStyle::Get().GetBrush("AdvancedRenamer.Style.BackgroundBorder"))
		.Content()
		[
			SNew(SVerticalBox)

			//Add Prefix
			+ SVerticalBox::Slot()
			.Padding(SectionContentFirstEntryPadding)
			.AutoHeight()
			[
				CreateAddPrefix()
			]

			// Add Suffix
			+ SVerticalBox::Slot()
			.Padding(SectionContentMiddleEntriesPadding)
			.AutoHeight()
			[
				CreateAddSuffix()
			]
		];
}

void FAdvancedRenamerAddPrefixSuffixSection::ResetToDefault()
{
	PrefixText = FText::GetEmpty();
	SuffixText = FText::GetEmpty();
}

TSharedRef<SWidget> FAdvancedRenamerAddPrefixSuffixSection::CreateAddPrefix()
{
	using namespace AdvancedRenamerSlateUtils::Default;
	
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(FirstWidgetPadding)
		[
			SNew(SBox)
			.WidthOverride(70.f)
			[
				SNew(STextBlock)
				.Font(FAdvancedRenamerStyle::Get().GetFontStyle("AdvancedRenamer.Style.RegularFont"))
				.Text(LOCTEXT("AR_AddPrefix", "Add Prefix"))
			]
		]
		
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.FillWidth(1.f)
		.Padding(LastWidgetPadding)
		[
			SAssignNew(PrefixTextBox, SEditableTextBox)
			.Font(FAdvancedRenamerStyle::Get().GetFontStyle("AdvancedRenamer.Style.RegularFont"))
			.HintText(LOCTEXT("AR_PrefixHint", "New Prefix"))
			.Text(this, &FAdvancedRenamerAddPrefixSuffixSection::GetPrefixText)
			.OnTextChanged(this, &FAdvancedRenamerAddPrefixSuffixSection::OnPrefixChanged)
		];
}

TSharedRef<SWidget> FAdvancedRenamerAddPrefixSuffixSection::CreateAddSuffix()
{
	using namespace AdvancedRenamerSlateUtils::Default;

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(FirstWidgetPadding)
		[
			SNew(SBox)
			.WidthOverride(70.f)
			[
				SNew(STextBlock)
				.Font(FAdvancedRenamerStyle::Get().GetFontStyle("AdvancedRenamer.Style.RegularFont"))
				.Text(LOCTEXT("BR_AddSuffix", "Add Suffix"))
			]
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.FillWidth(1.f)
		.Padding(LastWidgetPadding)
		[
			SAssignNew(SuffixTextBox, SEditableTextBox)
			.Font(FAdvancedRenamerStyle::Get().GetFontStyle("AdvancedRenamer.Style.RegularFont"))
			.HintText(LOCTEXT("AR_SuffixHint", "New Suffix"))
			.Text(this, &FAdvancedRenamerAddPrefixSuffixSection::GetSuffixText)
			.OnTextChanged(this, &FAdvancedRenamerAddPrefixSuffixSection::OnSuffixChanged)
		];
}

FText FAdvancedRenamerAddPrefixSuffixSection::GetPrefixText() const
{
	return PrefixText;
}

FText FAdvancedRenamerAddPrefixSuffixSection::GetSuffixText() const
{
	return SuffixText;
}

void FAdvancedRenamerAddPrefixSuffixSection::OnPrefixChanged(const FText& InNewText)
{
	PrefixText = InNewText;
	MarkRenamerDirty();
}

void FAdvancedRenamerAddPrefixSuffixSection::OnSuffixChanged(const FText& InNewText)
{
	SuffixText = InNewText;
	MarkRenamerDirty();
}

bool FAdvancedRenamerAddPrefixSuffixSection::CanApplyAddPrefixOperation()
{
	return !PrefixText.IsEmpty();
}

bool FAdvancedRenamerAddPrefixSuffixSection::CanApplyAddSuffixOperation()
{
	return !SuffixText.IsEmpty();
}

void FAdvancedRenamerAddPrefixSuffixSection::ApplyAddPrefixOperation(FString& OutOriginalName)
{
	OutOriginalName = PrefixText.ToString() + OutOriginalName;
}

void FAdvancedRenamerAddPrefixSuffixSection::ApplyAddSuffixOperation(FString& OutOriginalName)
{
	OutOriginalName = OutOriginalName + SuffixText.ToString();
}

void FAdvancedRenamerAddPrefixSuffixSection::ApplyAddPrefixSuffixNumberOperation(FString& OutOriginalName)
{
	if (CanApplyAddPrefixOperation())
	{
		ApplyAddPrefixOperation(OutOriginalName);
	}

	if (CanApplyAddSuffixOperation())
	{
		ApplyAddSuffixOperation(OutOriginalName);
	}
}

#undef LOCTEXT_NAMESPACE
