// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AdvancedRenamerSectionBase.h"
#include "Styling/SlateTypes.h"

class FString;
class SCheckBox;
class SEditableTextBox;
template <typename InNumericType>
class SSpinBox;
class SWidget;

class FAdvancedRenamerAddPrefixSuffixSection : public FAdvancedRenamerSectionBase
{
public:
	FAdvancedRenamerAddPrefixSuffixSection();

	virtual ~FAdvancedRenamerAddPrefixSuffixSection() = default;

	/** Init the given section */
	virtual void Init(TSharedRef<IAdvancedRenamer> InRenamer) override;

	/** Return the widget for this section*/
	virtual TSharedRef<SWidget> GetWidget() override;

	/** Reset all values of the section to the default ones */
	virtual void ResetToDefault() override;

private:
	/** Create the AddPrefix widget */
	TSharedRef<SWidget> CreateAddPrefix();

	/** Create the AddSuffix widget */
	TSharedRef<SWidget> CreateAddSuffix();

	/** Get the Prefix text */
	FText GetPrefixText() const;

	/** Get the Suffix text */
	FText GetSuffixText() const;

	/** Called when the Prefix text change */
	void OnPrefixChanged(const FText& InNewText);

	/** Called when the Suffix text change */
	void OnSuffixChanged(const FText& InNewText);

	/** Whether the AddPrefix operation can be executed */
	bool CanApplyAddPrefixOperation();

	/** Whether the AddSuffix operation can be executed */
	bool CanApplyAddSuffixOperation();

	/** Execute logic for the AddPrefix */
	void ApplyAddPrefixOperation(FString& OutOriginalName);

	/** Execute logic for the AddSuffix */
	void ApplyAddSuffixOperation(FString& OutOriginalName);

	/** Execute logic for this section */
	void ApplyAddPrefixSuffixNumberOperation(FString& OutOriginalName);

private:
	/** AddPrefix TextBox */
	TSharedPtr<SEditableTextBox> PrefixTextBox;
	
	/** AddSuffix TextBox */
	TSharedPtr<SEditableTextBox> SuffixTextBox;

	/** AddPrefix Text */
	FText PrefixText;

	/** AddSuffix text */
	FText SuffixText;
};
