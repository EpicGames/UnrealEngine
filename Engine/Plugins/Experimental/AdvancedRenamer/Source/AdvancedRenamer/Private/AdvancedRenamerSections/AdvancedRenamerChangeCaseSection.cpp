// Copyright Epic Games, Inc. All Rights Reserved.

#include "AdvancedRenamerChangeCaseSection.h"

#include "AdvancedRenamerStyle.h"
#include "IAdvancedRenamer.h"
#include "Utils/AdvancedRenamerSlateUtils.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "AdvancedRenamerChangeCaseSection"

FAdvancedRenamerChangeCaseSection::FAdvancedRenamerChangeCaseSection()
{
	FAdvancedRenamerChangeCaseSection::ResetToDefault();
}

void FAdvancedRenamerChangeCaseSection::Init(TSharedRef<IAdvancedRenamer> InRenamer)
{
	FAdvancedRenamerSectionBase::Init(InRenamer);
	Section.SectionName = TEXT("ChangeCase");
	Section.OnOperationExecuted().BindSP(this, &FAdvancedRenamerChangeCaseSection::ApplyChangeCaseSection);
	Section.OnAfterOperationExecutionEnded().BindSP(this, &FAdvancedRenamerChangeCaseSection::ResetButtonClicked);
	InRenamer->AddSection(Section);
}

TSharedRef<SWidget> FAdvancedRenamerChangeCaseSection::GetWidget()
{
	using namespace AdvancedRenamerSlateUtils::Default;

	// Add click logic
	return SNew(SBorder)
		.BorderImage(FAdvancedRenamerStyle::Get().GetBrush("AdvancedRenamer.Style.BackgroundBorder"))
		.Content()
		[
			SNew(SVerticalBox)

			//Title
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Left)
			.Padding(SectionContentFirstEntryPadding)
			.AutoHeight()
			[
				SNew(STextBlock)
				.Font(FAdvancedRenamerStyle::Get().GetFontStyle("AdvancedRenamer.Style.TitleFont"))
				.Text(LOCTEXT("AR_ChangeCaseTitle", "Change Case"))
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(SectionContentMiddleEntriesPadding)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.FillWidth(0.25f)
				.VAlign(VAlign_Center)
				.Padding(ChangeCaseFirstButtonPadding)
				[
					SNew(SBox)
					.HeightOverride(25)
					[
						SNew(SButton)
						.ContentPadding(FMargin(-2.f, -1.f))
						.OnClicked(this, &FAdvancedRenamerChangeCaseSection::OnChangeCaseButtonClicked, EAdvancedRenamerChangeCaseType::SwapFirst)
						.Content()
						[
							SNew(SBox)
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.Justification(ETextJustify::Center)
								.Text(LOCTEXT("AR_SwapFirst", "Swap First"))
							]
						]
					]
				]

				+ SHorizontalBox::Slot()
				.FillWidth(0.25f)
				.VAlign(VAlign_Center)
				.Padding(ChangeCaseMiddleButtonsPadding)
				[
					SNew(SBox)
					.HeightOverride(25)
					[
						SNew(SButton)
						.ContentPadding(FMargin(-2.f, -1.f))
						.OnClicked(this, &FAdvancedRenamerChangeCaseSection::OnChangeCaseButtonClicked, EAdvancedRenamerChangeCaseType::SwapAll)
						.Content()
						[
							SNew(SBox)
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.Justification(ETextJustify::Center)
								.Text(LOCTEXT("AR_SwapAll", "Swap All"))
							]
						]
					]
				]

				+ SHorizontalBox::Slot()
				.FillWidth(0.25f)
				.VAlign(VAlign_Center)
				.Padding(ChangeCaseMiddleButtonsPadding)
				[
					SNew(SBox)
					.HeightOverride(25)
					[
						SNew(SButton)
						.ContentPadding(FMargin(-2.f, -1.f))
						.OnClicked(this, &FAdvancedRenamerChangeCaseSection::OnChangeCaseButtonClicked, EAdvancedRenamerChangeCaseType::AllLower)
						.Content()
						[
							SNew(SBox)
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.Justification(ETextJustify::Center)
								.Text(LOCTEXT("AR_AllLower", "All Lower"))
							]
						]
					]
				]
				
				+ SHorizontalBox::Slot()
				.FillWidth(0.25f)
				.VAlign(VAlign_Center)
    		    .Padding(ChangeCaseLastButtonPadding)
    		    [
    				SNew(SBox)
					.HeightOverride(25)
					[
    		    		SNew(SButton)
    		    		.ContentPadding(FMargin(-2.f, -1.f))
    		    		.OnClicked(this, &FAdvancedRenamerChangeCaseSection::OnChangeCaseButtonClicked, EAdvancedRenamerChangeCaseType::AllUpper)
    		    		.Content()
    		    		[
    		    			SNew(SBox)
							.VAlign(VAlign_Center)
							[
    		    				SNew(STextBlock)
    		    				.Justification(ETextJustify::Center)
    		    				.Text(LOCTEXT("AR_AllUpper", "All Upper"))
    		    			]
    		    		]
    		    	]
    		    ]
			]
		];
}

void FAdvancedRenamerChangeCaseSection::ResetToDefault()
{
	bButtonWasClicked = false;
	ChangeCaseType = EAdvancedRenamerChangeCaseType::SwapFirst;
}

FReply FAdvancedRenamerChangeCaseSection::OnChangeCaseButtonClicked(EAdvancedRenamerChangeCaseType InNewValue)
{
	ChangeCaseType = InNewValue;
	bButtonWasClicked = true;
	MarkRenamerDirty();
	return FReply::Handled();
}

bool FAdvancedRenamerChangeCaseSection::CanApplyChangeCaseSection()
{
	return bButtonWasClicked;
}

void FAdvancedRenamerChangeCaseSection::ApplySwapFirst(FString& OutOriginalName)
{
	if (!OutOriginalName.IsEmpty())
	{
		TCHAR& First = OutOriginalName[0];

		if (FChar::IsUpper(First))
		{
			First = FChar::ToLower(First);
		}
		else if (FChar::IsLower(First))
		{
			First = FChar::ToUpper(First);
		}
	}
}

void FAdvancedRenamerChangeCaseSection::ApplySwapAll(FString& OutOriginalName)
{
	for (int32 CurrentNameIndex = 0; CurrentNameIndex < OutOriginalName.GetCharArray().Num() - 1; CurrentNameIndex++)
	{
		TCHAR& CurrentChar = OutOriginalName[CurrentNameIndex];
		if (FChar::IsUpper(CurrentChar))
		{
			CurrentChar = FChar::ToLower(CurrentChar);
		}
		else if (FChar::IsLower(CurrentChar))
		{
			CurrentChar = FChar::ToUpper(CurrentChar);
		}
	}
}

void FAdvancedRenamerChangeCaseSection::ApplyAllLower(FString& OutOriginalName)
{
	OutOriginalName.ToLowerInline();
}

void FAdvancedRenamerChangeCaseSection::ApplyAllUpper(FString& OutOriginalName)
{
	OutOriginalName.ToUpperInline();
}

void FAdvancedRenamerChangeCaseSection::ResetButtonClicked()
{
	bButtonWasClicked = false;
}

void FAdvancedRenamerChangeCaseSection::ApplyChangeCaseSection(FString& OutOriginalName)
{
	if (bButtonWasClicked)
	{
		switch (ChangeCaseType)
		{
		case EAdvancedRenamerChangeCaseType::SwapFirst:
			ApplySwapFirst(OutOriginalName);
			break;
		case EAdvancedRenamerChangeCaseType::SwapAll:
			ApplySwapAll(OutOriginalName);
			break;
		case EAdvancedRenamerChangeCaseType::AllLower:
			ApplyAllLower(OutOriginalName);
			break;
		case EAdvancedRenamerChangeCaseType::AllUpper:
			ApplyAllUpper(OutOriginalName);
			break;
		default:
			break;
		}
	}
}

#undef LOCTEXT_NAMESPACE
