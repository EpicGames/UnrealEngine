// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AdvancedRenamerSectionBase.h"
#include "Input/Reply.h"

class FString;
class SWidget;

/**
 * Change case types
 */
enum class EAdvancedRenamerChangeCaseType : uint8
{
	SwapFirst,
	SwapAll,
	AllLower,
	AllUpper
};

/**
 * ChangeCase section
 */
class FAdvancedRenamerChangeCaseSection : public FAdvancedRenamerSectionBase
{
public:
	FAdvancedRenamerChangeCaseSection();

	virtual ~FAdvancedRenamerChangeCaseSection() = default;

	/** Init the given section */
	virtual void Init(TSharedRef<IAdvancedRenamer> InRenamer) override;

	/** Return the widget for this section*/
	virtual TSharedRef<SWidget> GetWidget() override;

	/** Reset all values of the section to the default ones */
	virtual void ResetToDefault() override;

private:
	/** Called when one of the CaseChange buttons is clicked */
	FReply OnChangeCaseButtonClicked(EAdvancedRenamerChangeCaseType InNewValue);

	/** Whether or not the ChangeCase operation can be executed */
	bool CanApplyChangeCaseSection();

	/** Execute logic for the SwapFirst */
	void ApplySwapFirst(FString& OutOriginalName);

	/** Execute logic for the SwapAll */
	void ApplySwapAll(FString& OutOriginalName);

	/** Execute logic for the AllLower */
	void ApplyAllLower(FString& OutOriginalName);

	/** Execute logic for the AllUpper */
	void ApplyAllUpper(FString& OutOriginalName);

	/** Reset the current button clicked and disable the ChangeCase until the next time a button is clicked */
	void ResetButtonClicked();

	/** Execute logic for this section */
	void ApplyChangeCaseSection(FString& OutOriginalName);

private:
	/** ChangeCase type to execute */
	EAdvancedRenamerChangeCaseType ChangeCaseType;
	
	/** Whether a button was clicked and the Renamer needs to apply the ChangeCase */
	bool bButtonWasClicked;
};
