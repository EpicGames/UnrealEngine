// Copyright Epic Games, Inc. All Rights Reserved.

#include "AdvancedRenamerNumberingSection.h"

#include "AdvancedRenamerStyle.h"
#include "IAdvancedRenamer.h"
#include "Utils/AdvancedRenamerSlateUtils.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"

#define LOCTEXT_NAMESPACE "AdvancedRenamerNumberingSection"

TArray<FText> FAdvancedRenamerNumberingSection::ComboBoxTextOptions = { LOCTEXT("AR_FirstFormat", "0"), LOCTEXT("AR_SecondFormat", "00"), LOCTEXT("AR_ThirdFormat", "000"), LOCTEXT("AR_FourthFormat", "0000") };

namespace AdvancedRenamerNumberingSectionUtils
{
	static constexpr TCHAR FirstUpperLetter = TEXT('A');
	static constexpr TCHAR FirstLowerLetter = TEXT('a');
	static constexpr int32 LetteringStep = 1;

	/** Return the reference letter to use based on the given parameter */
	TCHAR GetReferenceLetter(bool bInIsLower)
	{
		if (bInIsLower)
		{
			return FirstLowerLetter;
		}
		return FirstUpperLetter;
	}
}

FAdvancedRenamerNumberingSection::FAdvancedRenamerNumberingSection()
{
	FAdvancedRenamerNumberingSection::ResetToDefault();
}

void FAdvancedRenamerNumberingSection::Init(TSharedRef<IAdvancedRenamer> InRenamer)
{
	FAdvancedRenamerSectionBase::Init(InRenamer);

	ComboBoxSourceOptions.Empty(ComboBoxTextOptions.Num());
	for (int32 Index = 0; Index < ComboBoxTextOptions.Num(); Index++)
	{
		ComboBoxSourceOptions.Add(MakeShared<int32>(Index));
	}

	Section.SectionName = TEXT("NumberingLettering");
	Section.OnBeforeOperationExecutionStart().BindSP(this, &FAdvancedRenamerNumberingSection::ResetCurrentNumbering);
	Section.OnOperationExecuted().BindSP(this, &FAdvancedRenamerNumberingSection::ApplyNumbering);
	InRenamer->AddSection(Section);
}

TSharedRef<SWidget> FAdvancedRenamerNumberingSection::GetWidget()
{
	using namespace AdvancedRenamerSlateUtils::Default;

	return SNew(SBorder)
		.BorderImage(FAdvancedRenamerStyle::Get().GetBrush("AdvancedRenamer.Style.BackgroundBorder"))
		.Content()
		[
			SNew(SVerticalBox)

			// Numbering Header
			+ SVerticalBox::Slot()
			.Padding(SectionContentFirstEntryPadding)
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(FirstWidgetPadding)
				[
					SAssignNew(AddNumberingCheckBox, SCheckBox)
					.IsChecked(this, &FAdvancedRenamerNumberingSection::IsAddNumberingChecked)
					.OnCheckStateChanged(this, &FAdvancedRenamerNumberingSection::OnAddNumberingCheckBoxChanged)
				]

				+ SHorizontalBox::Slot()
				.Padding(FirstWidgetPadding)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font(FAdvancedRenamerStyle::Get().GetFontStyle("AdvancedRenamer.Style.RegularFont"))
					.Text(LOCTEXT("AR_AddNumberingLettering", "Auto Increment"))
				]

				+ SHorizontalBox::Slot()
				.Padding(LastWidgetPadding)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				[
					SNew(SBox)
					.HeightOverride(25.f)
					.MinDesiredWidth(160.f)
					[
						SNew(SSegmentedControl<EAdvancedRenamerNumberingType>)
						.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
						.IsEnabled(this, &FAdvancedRenamerNumberingSection::IsAddNumberingEnabled)
						.UniformPadding(FMargin(2.f, 0.f))
						.SupportsEmptySelection(false)
						.SupportsMultiSelection(false)
						.Value(this, &FAdvancedRenamerNumberingSection::GetAddNumberingType)
						.OnValueChanged(this, &FAdvancedRenamerNumberingSection::OnAddNumberingTypeChanged)
									
						+ SSegmentedControl<EAdvancedRenamerNumberingType>::Slot(EAdvancedRenamerNumberingType::Number)
						.Text(LOCTEXT("AR_NumberingType", "#"))
									
						+ SSegmentedControl<EAdvancedRenamerNumberingType>::Slot(EAdvancedRenamerNumberingType::Letter)
						.Text(LOCTEXT("AR_LetteringType", "Aa"))
					]
				]
			]

			// Add Number/Letter and Format
			+ SVerticalBox::Slot()
			.Padding(SectionContentFirstEntryPadding)
			.AutoHeight()
			[
				SNew(SBox)
				.HeightOverride(24.5f)
				[
					SNew(SHorizontalBox)

					// Formatting
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(FirstWidgetPadding)
					.FillWidth(0.3f)
					[
						SNew(SHorizontalBox)
						.Visibility(this, &FAdvancedRenamerNumberingSection::IsAddNumberingTypeNumber)

						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.Padding(FirstWidgetPadding)
						.AutoWidth()
						[
							SNew(STextBlock)
							.Font(FAdvancedRenamerStyle::Get().GetFontStyle("AdvancedRenamer.Style.RegularFont"))
							.Text(LOCTEXT("AR_FormattingLabel", "Format"))
						]

						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.FillWidth(1.f)
						[
							SAssignNew(FormattingComboBox, SComboBox<TSharedPtr<int32>>)
							.IsEnabled(this, &FAdvancedRenamerNumberingSection::IsAddNumberingEnabled)
							.OptionsSource(&ComboBoxSourceOptions)
							.InitiallySelectedItem(ComboBoxSourceOptions[0])
							.ToolTipText(LOCTEXT("AR_FormatTooltip", "Select the format you prefer for the numbering"))
							.OnGenerateWidget(this, &FAdvancedRenamerNumberingSection::OnGenerateFormatWidget)
							.OnSelectionChanged(this, &FAdvancedRenamerNumberingSection::OnFormatSelectionChanged)
							.Content()
							[
								SNew(STextBlock)
								.Text(this, &FAdvancedRenamerNumberingSection::GetCurrentFormatText)
							]
						]
					]

					// Start
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(MiddleWidgetPadding)
					.FillWidth(0.3f)
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.Padding(FirstWidgetPadding)
						.AutoWidth()
						[
							SNew(STextBlock)
							.Font(FAdvancedRenamerStyle::Get().GetFontStyle("AdvancedRenamer.Style.RegularFont"))
							.Text(LOCTEXT("AR_Start", "Start"))
						]

						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.FillWidth(1.f)
						[
							SNew(SWidgetSwitcher)
							.WidgetIndex(this, &FAdvancedRenamerNumberingSection::GetNumberingIndex)

							// Add Number
							+ SWidgetSwitcher::Slot()
							[
								SAssignNew(AddNumberStartSpinBox, SSpinBox<int32>)
								.Style(&FAppStyle::Get(), "Menu.SpinBox")
								.Font(FAdvancedRenamerStyle::Get().GetFontStyle("AdvancedRenamer.Style.RegularFont"))
								.MinValue(0)
								.Value(this, &FAdvancedRenamerNumberingSection::GetAddNumberValue)
								.IsEnabled(this, &FAdvancedRenamerNumberingSection::IsAddNumberingEnabled)
								.OnValueChanged(this, &FAdvancedRenamerNumberingSection::OnAddNumberChanged)
							]

							// Add Letter
							+ SWidgetSwitcher::Slot()
							[
								SAssignNew(AddLetterStartTextBox, SEditableTextBox)
								.Font(FAdvancedRenamerStyle::Get().GetFontStyle("AdvancedRenamer.Style.RegularFont"))
								.OnVerifyTextChanged(this, &FAdvancedRenamerNumberingSection::OnLetteringVerifyText)
								.Text(this, &FAdvancedRenamerNumberingSection::GetAddLetterText)
								.IsEnabled(this, &FAdvancedRenamerNumberingSection::IsAddNumberingEnabled)
								.OnTextChanged(this, &FAdvancedRenamerNumberingSection::OnAddLetterChanged)
							]
						]
					]

					// Step
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(LastWidgetPadding)
					.FillWidth(0.3f)
					[
						SNew(SHorizontalBox)
						.Visibility(this, &FAdvancedRenamerNumberingSection::IsAddNumberingTypeNumber)

						+ SHorizontalBox::Slot()
						.Padding(FirstWidgetPadding)
						.VAlign(VAlign_Center)
						.AutoWidth()
						[
							SNew(STextBlock)
							.Font(FAdvancedRenamerStyle::Get().GetFontStyle("AdvancedRenamer.Style.RegularFont"))
							.Text(LOCTEXT("AR_Step", "Step"))
						]

						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.FillWidth(1.f)
						[
							SAssignNew(AddNumberStepSpinBox, SSpinBox<int32>)
							.Style(&FAppStyle::Get(), "Menu.SpinBox")
							.Font(FAdvancedRenamerStyle::Get().GetFontStyle("AdvancedRenamer.Style.RegularFont"))
							.MinValue(1)
							.MaxValue(99)
							.Value(this, &FAdvancedRenamerNumberingSection::GetAddNumberingStepValue)
							.IsEnabled(this, &FAdvancedRenamerNumberingSection::IsAddNumberingEnabled)
							.OnValueChanged(this, &FAdvancedRenamerNumberingSection::OnAddNumberingStepChanged)
						]
					]
				]
			]
		];
}

void FAdvancedRenamerNumberingSection::ResetToDefault()
{
	bIsAddNumberingSectionEnabled = false;
	bIsAddLetteringInputCorrect = true;
	AddNumberingType = EAdvancedRenamerNumberingType::Number;
	AddNumberingWidgetSwitcherIndex = 0;
	AddNumberValue = 0;
	AddLetterText = LOCTEXT("AR_AddLetter", "A");
	AddNumberingStepValue = 1;
	CurrentAddNumberValue = 1;
	CurrentAddLetterString = TEXT("");
	CurrentFormatChosen = 0;
	if (FormattingComboBox.IsValid() && !ComboBoxSourceOptions.IsEmpty())
	{
		FormattingComboBox->SetSelectedItem(ComboBoxSourceOptions[0]);
	}
}

ECheckBoxState FAdvancedRenamerNumberingSection::IsAddNumberingChecked() const
{
	return bIsAddNumberingSectionEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

bool FAdvancedRenamerNumberingSection::IsAddNumberingEnabled() const
{
	return bIsAddNumberingSectionEnabled;
}

EVisibility FAdvancedRenamerNumberingSection::IsAddNumberingTypeNumber() const
{
	return AddNumberingType == EAdvancedRenamerNumberingType::Number ? EVisibility::Visible : EVisibility::Collapsed;
}

EAdvancedRenamerNumberingType FAdvancedRenamerNumberingSection::GetAddNumberingType() const
{
	return AddNumberingType;
}

FText FAdvancedRenamerNumberingSection::GetCurrentFormatText() const
{
	if (ComboBoxTextOptions.IsValidIndex(CurrentFormatChosen))
	{
		return ComboBoxTextOptions[CurrentFormatChosen];
	}
	return FText::GetEmpty();
}

int32 FAdvancedRenamerNumberingSection::GetNumberingIndex() const
{
	return AddNumberingWidgetSwitcherIndex;
}

int32 FAdvancedRenamerNumberingSection::GetAddNumberValue() const
{
	return AddNumberValue;
}

FText FAdvancedRenamerNumberingSection::GetAddLetterText() const
{
	return AddLetterText;
}

int32 FAdvancedRenamerNumberingSection::GetAddNumberingStepValue() const
{
	return AddNumberingStepValue;
}

FText FAdvancedRenamerNumberingSection::GetFormatTextForIndex(TSharedPtr<int32> InFormatIndex) const
{
	if (InFormatIndex.IsValid())
	{
		int32 Index = *InFormatIndex.Get();
		if (ComboBoxTextOptions.IsValidIndex(Index))
		{
			return ComboBoxTextOptions[Index];
		}
	}
	return FText::GetEmpty();
}

FString FAdvancedRenamerNumberingSection::GetFormattedNumber() const
{
	if (CurrentFormatChosen == 0)
	{
		return FString::Printf(TEXT("%01d"), CurrentAddNumberValue);
	}
	if (CurrentFormatChosen == 1)
	{
		return FString::Printf(TEXT("%02d"), CurrentAddNumberValue);
	}
	if (CurrentFormatChosen == 2)
	{
		return FString::Printf(TEXT("%03d"), CurrentAddNumberValue);
	}
	if (CurrentFormatChosen == 3)
	{
		return FString::Printf(TEXT("%04d"), CurrentAddNumberValue);
	}

	return TEXT("Error: Format Not Valid");
}

TSharedRef<SWidget> FAdvancedRenamerNumberingSection::OnGenerateFormatWidget(TSharedPtr<int32> InOption) const
{
	using namespace AdvancedRenamerSlateUtils::Default;

	return SNew(SBox)
		.Padding(VerticalPadding)
		[
			SNew(STextBlock)
			.Text(this, &FAdvancedRenamerNumberingSection::GetFormatTextForIndex, InOption)
		];
}

void FAdvancedRenamerNumberingSection::OnFormatSelectionChanged(TSharedPtr<int32> InNewFormat, ESelectInfo::Type InSelectInfo)
{
	if (InNewFormat.IsValid())
	{
		CurrentFormatChosen = *InNewFormat.Get();
	}
	else
	{
		// Set the format to -1 so that the widget will display an error text showing that an error occured
		CurrentFormatChosen = -1;
	}
	MarkRenamerDirty();
}

void FAdvancedRenamerNumberingSection::OnAddNumberingCheckBoxChanged(ECheckBoxState InNewState)
{
	bIsAddNumberingSectionEnabled = InNewState == ECheckBoxState::Checked;
	MarkRenamerDirty();
}

void FAdvancedRenamerNumberingSection::OnAddNumberingTypeChanged(EAdvancedRenamerNumberingType InNewValue)
{
	AddNumberingType = InNewValue;
	switch (AddNumberingType)
	{
		case EAdvancedRenamerNumberingType::Number:
			AddNumberingWidgetSwitcherIndex = 0;
			break;

		case EAdvancedRenamerNumberingType::Letter:
			AddNumberingWidgetSwitcherIndex = 1;
			break;
	}
	MarkRenamerDirty();
}

void FAdvancedRenamerNumberingSection::OnAddNumberChanged(int32 InNewValue)
{
	AddNumberValue = InNewValue;
	MarkRenamerDirty();
}

void FAdvancedRenamerNumberingSection::OnAddLetterChanged(const FText& InNewText)
{
	AddLetterText = InNewText;
	MarkRenamerDirty();
}

bool FAdvancedRenamerNumberingSection::OnLetteringVerifyText(const FText& InNewText, FText& OutErrorText)
{
	FString NewString = InNewText.ToString();

	for (int32 StringIndex = 0; StringIndex < NewString.Len(); StringIndex++)
	{
		if (!FChar::IsAlpha(NewString[StringIndex]))
		{
			OutErrorText = LOCTEXT("AR_LetterInputNotValid", "Only alphabetic character are allowed as input");
			bIsAddLetteringInputCorrect = false;
			return false;
		}
	}

	bIsAddLetteringInputCorrect = true;
	return true;
}

void FAdvancedRenamerNumberingSection::OnAddNumberingStepChanged(int32 InNewValue)
{
	AddNumberingStepValue = InNewValue;
	MarkRenamerDirty();
}

bool FAdvancedRenamerNumberingSection::CanApplyAddNumberOperation() const
{
	return AddNumberingType == EAdvancedRenamerNumberingType::Number;
}

bool FAdvancedRenamerNumberingSection::CanApplyAddLetterOperation() const
{
	return AddNumberingType == EAdvancedRenamerNumberingType::Letter && bIsAddLetteringInputCorrect;
}

void FAdvancedRenamerNumberingSection::ApplyAddNumberOperation(FString& OutOriginalName)
{
	OutOriginalName += GetFormattedNumber();
	CurrentAddNumberValue += AddNumberingStepValue;
}

void FAdvancedRenamerNumberingSection::ApplyAddLetterOperation(FString& OutOriginalName)
{
	using namespace AdvancedRenamerNumberingSectionUtils;

	if (CurrentAddLetterString.IsEmpty())
	{
		return;
	}

	OutOriginalName += CurrentAddLetterString;

	int32 CountToUse = LetteringStep;

	for (int32 CurrentLetteringChar = (CurrentAddLetterString.Len() - 1); CurrentLetteringChar >= 0; --CurrentLetteringChar)
	{
		if (CountToUse == 0)
		{
			break;
		}

		TCHAR& CurrentChar = CurrentAddLetterString[CurrentLetteringChar];
		TCHAR LetterToUseAsReference = GetReferenceLetter(FChar::IsLower(CurrentChar));

		int32 Difference = (CurrentChar - LetterToUseAsReference + CountToUse);
		CurrentChar = LetterToUseAsReference + (Difference % 26);
		CountToUse = (float)Difference / 26.f;

		if (!CurrentAddLetterString.IsValidIndex(CurrentLetteringChar - 1) && CountToUse > 0)
		{
			CurrentAddLetterString.InsertAt(0, LetterToUseAsReference);
		}
	}
}

void FAdvancedRenamerNumberingSection::ResetCurrentNumbering()
{
	CurrentAddNumberValue = AddNumberValue;
	CurrentAddLetterString = AddLetterText.ToString();
}

void FAdvancedRenamerNumberingSection::ApplyNumbering(FString& OutOriginalName)
{
	if (bIsAddNumberingSectionEnabled)
	{
		if (CanApplyAddNumberOperation())
		{
			ApplyAddNumberOperation(OutOriginalName);
		}
		else if (CanApplyAddLetterOperation())
		{
			ApplyAddLetterOperation(OutOriginalName);
		}
	}
}

#undef LOCTEXT_NAMESPACE
