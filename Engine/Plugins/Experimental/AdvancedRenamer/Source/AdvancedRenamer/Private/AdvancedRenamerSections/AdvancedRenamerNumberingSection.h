// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AdvancedRenamerSectionBase.h"
#include "Layout/Visibility.h"
#include "Styling/SlateTypes.h"

class SCheckBox;
template <typename InOptionType>
class SComboBox;
class SEditableTextBox;
template <typename InNumericType>
class SSpinBox;
class SWidget;

enum class EAdvancedRenamerNumberingType
{
	Number,
	Letter
};

class FAdvancedRenamerNumberingSection : public FAdvancedRenamerSectionBase
{
public:
	FAdvancedRenamerNumberingSection();

	virtual ~FAdvancedRenamerNumberingSection() = default;

	/** Init the given section */
	virtual void Init(TSharedRef<IAdvancedRenamer> InRenamer) override;

	/** Return the widget for this section*/
	virtual TSharedRef<SWidget> GetWidget() override;

	/** Reset all values of the section to the default ones */
	virtual void ResetToDefault() override;

private:
	/** If checked the section will be enabled */
	ECheckBoxState IsAddNumberingChecked() const;

	/** Whether the section is enabled */
	bool IsAddNumberingEnabled() const;

	/** Visible if the type is Number, Collapsed otherwise */
	EVisibility IsAddNumberingTypeNumber() const;

	/** Get the type of the AddNumbering, either by Number or by Letter */
	EAdvancedRenamerNumberingType GetAddNumberingType() const;

	/** Get the current Text Format */
	FText GetCurrentFormatText() const;

	/** Get the Index of the WidgetSwitcher to either show the Number or the Letter */
	int32 GetNumberingIndex() const;

	/** Get the starting Number of the AddNumber SpinBox */
	int32 GetAddNumberValue() const;

	/** Get the starting Letter of the AddLetter TextBox */
	FText GetAddLetterText() const;

	/** Get the starting Number of the AddNumbering Step SpinBox */
	int32 GetAddNumberingStepValue() const;

	/** Get the Text Format for the given InFormatIndex */
	FText GetFormatTextForIndex(TSharedPtr<int32> InFormatIndex) const;

	/** Get the formatted Number Text */
	FString GetFormattedNumber() const;

	/** Generate the Format option widget for the ComboBox */
	TSharedRef<SWidget> OnGenerateFormatWidget(TSharedPtr<int32> InOption) const;

	/** Called when the Format selected change */
	void OnFormatSelectionChanged(TSharedPtr<int32> InNewFormat, ESelectInfo::Type InSelectInfo);

	/** Called when the AddNumbering CheckBox state change */
	void OnAddNumberingCheckBoxChanged(ECheckBoxState InNewState);

	/** Called when the AddNumbering type changed, it will be either by Number or Letter */
	void OnAddNumberingTypeChanged(EAdvancedRenamerNumberingType InNewValue);

	/** Called when the AddNumber number change */
	void OnAddNumberChanged(int32 InNewValue);

	/** Called when the AddLetter letter change */
	void OnAddLetterChanged(const FText& InNewText);

	/** Called to verify that the Letter input is correct and allowed */
	bool OnLetteringVerifyText(const FText& InNewText, FText& OutErrorText);

	/** Called when the AddNumbering Step number change */
	void OnAddNumberingStepChanged(int32 InNewValue);

	/** Whether the AddNumber operation can be executed */
	bool CanApplyAddNumberOperation() const;

	/** Whether the AddLetter operation can be executed */
	bool CanApplyAddLetterOperation() const;

	/** Execute logic for the AddNumber */
	void ApplyAddNumberOperation(FString& OutOriginalName);

	/** Execute logic for the AddLetter */
	void ApplyAddLetterOperation(FString& OutOriginalName);

	/** Reset current numbering */
	void ResetCurrentNumbering();

	/** Execute logic for this section */
	void ApplyNumbering(FString& OutOriginalName);

private:
	/** Section enabler CheckBox */
	TSharedPtr<SCheckBox> AddNumberingCheckBox;

	/** Starting Number SpinBox for the AddNumber */
	TSharedPtr<SSpinBox<int32>> AddNumberStartSpinBox;

	/** Starting Letter TextBox for the AddLetter */
	TSharedPtr<SEditableTextBox> AddLetterStartTextBox;

	/** Step Number SpinBox for the AddNumbering section */
	TSharedPtr<SSpinBox<int32>> AddNumberStepSpinBox;

	/** Step Number SpinBox for the AddNumbering section */
	TSharedPtr<SComboBox<TSharedPtr<int32>>> FormattingComboBox;

	/** AddNumbering section enabler, section is enabled if true */
	bool bIsAddNumberingSectionEnabled;

	/** AddNumbering section enabler, section is enabled if true */
	bool bIsAddLetteringInputCorrect;

	/** AddNumbering type, either Number or Letter */
	EAdvancedRenamerNumberingType AddNumberingType;

	/** Index of the WidgetSwitcher between Number and Letter */
	int32 AddNumberingWidgetSwitcherIndex;

	/** AddNumber start/stop value */
	int32 AddNumberValue;

	/** AddLetter start/stop value */
	FText AddLetterText;

	/** AddNumber step value */
	int32 AddNumberingStepValue;

	/** Current AddNumber value */
	int32 CurrentAddNumberValue;

	/** Current AddLetter value */
	FString CurrentAddLetterString;

	/** Current format chosen */
	int32 CurrentFormatChosen;

	/** Source option for the DropDown Menu of the ComboBox */
	TArray<TSharedPtr<int32>> ComboBoxSourceOptions;

	/** Text option based on the SourceOption */
	static TArray<FText> ComboBoxTextOptions;
};
