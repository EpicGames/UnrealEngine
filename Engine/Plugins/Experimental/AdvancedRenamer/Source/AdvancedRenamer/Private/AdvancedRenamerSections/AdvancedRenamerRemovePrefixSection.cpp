// Copyright Epic Games, Inc. All Rights Reserved.

#include "AdvancedRenamerRemovePrefixSection.h"

#include "IAdvancedRenamer.h"
#include "AdvancedRenamerStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "AdvancedRenamerRemovePrefixSection"

FAdvancedRenamerRemovePrefixSection::FAdvancedRenamerRemovePrefixSection()
{
	FAdvancedRenamerRemovePrefixSection::ResetToDefault();
}

void FAdvancedRenamerRemovePrefixSection::Init(TSharedRef<IAdvancedRenamer> InRenamer)
{
	FAdvancedRenamerSectionBase::Init(InRenamer);
	Section.SectionName = TEXT("RemovePrefix");
	Section.OnOperationExecuted().BindSP(this, &FAdvancedRenamerRemovePrefixSection::ApplyRemovePrefixOperation);
	InRenamer->AddSection(Section);
}

TSharedRef<SWidget> FAdvancedRenamerRemovePrefixSection::GetWidget()
{
	using namespace AdvancedRenamerSlateUtils::Default;

	return SNew(SBorder)
		.BorderImage(FAdvancedRenamerStyle::Get().GetBrush("AdvancedRenamer.Style.BackgroundBorder"))
		.Content()
		[
			SNew(SVerticalBox)

			// Remove Prefix CheckBox
			+ SVerticalBox::Slot()
			.Padding(SectionContentFirstEntryPadding)
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(FirstWidgetPadding)
				[
					SAssignNew(RemoveOldPrefixCheckBox, SCheckBox)
					.IsChecked(this, &FAdvancedRenamerRemovePrefixSection::IsRemoveOldPrefixChecked)
					.OnCheckStateChanged(this, &FAdvancedRenamerRemovePrefixSection::OnRemoveOldPrefixCheckBoxChanged)
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font(FAdvancedRenamerStyle::Get().GetFontStyle("AdvancedRenamer.Style.RegularFont"))
					.Text(LOCTEXT("AR_RemoveOldPrefix", "Remove Old Prefix"))
				]
			]

			// Remove Prefix
			+ SVerticalBox::Slot()
			.Padding(SectionContentMiddleEntriesPadding)
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(FirstWidgetPadding)
				[
					SNew(SBox)
					.HeightOverride(25.f)
					.MinDesiredWidth(160.f)
					[
						SNew(SSegmentedControl<EAdvancedRenamerRemoveOldType>)
						.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
						.IsEnabled(this, &FAdvancedRenamerRemovePrefixSection::IsRemoveOldPrefixEnabled)
						.UniformPadding(FMargin(2.f, 0.f))
						.SupportsEmptySelection(false)
						.SupportsMultiSelection(false)
						.Value(this, &FAdvancedRenamerRemovePrefixSection::GetRemoveOldPrefixType)
						.OnValueChanged(this, &FAdvancedRenamerRemovePrefixSection::OnRemoveOldPrefixTypeChanged)

						+ SSegmentedControl<EAdvancedRenamerRemoveOldType>::Slot(EAdvancedRenamerRemoveOldType::Separator)
						.Text(LOCTEXT("AR_PrefixSeparator", "Separator"))

						+ SSegmentedControl<EAdvancedRenamerRemoveOldType>::Slot(EAdvancedRenamerRemoveOldType::Chars)
						.Text(LOCTEXT("AR_PrefixFirstChar(s)", "First Char(s)"))
					]
				]
				+ SHorizontalBox::Slot()
				.Padding(LastWidgetPadding)
				[
					SNew(SWidgetSwitcher)
					.IsEnabled(this, &FAdvancedRenamerRemovePrefixSection::IsRemoveOldPrefixEnabled)
					.WidgetIndex(TAttribute<int32>::CreateLambda([this] () { return PrefixWidgetSwitcherIndex; }))

					+ SWidgetSwitcher::Slot()
					[
						SAssignNew(PrefixSeparatorTextBox, SEditableTextBox)
						.Font(FAdvancedRenamerStyle::Get().GetFontStyle("AdvancedRenamer.Style.RegularFont"))
						.Text(this, &FAdvancedRenamerRemovePrefixSection::GetPrefixSeparatorText)
						.OnVerifyTextChanged(this, &FAdvancedRenamerRemovePrefixSection::OnPrefixSeparatorVerifyTextChanged)
						.OnTextChanged(this, &FAdvancedRenamerRemovePrefixSection::OnPrefixSeparatorChanged)
					]

					+ SWidgetSwitcher::Slot()
					[
						SAssignNew(PrefixRemoveCharactersSpinBox, SSpinBox<uint8>)
						.Style(&FAppStyle::Get(), "Menu.SpinBox")
						.Font(FAdvancedRenamerStyle::Get().GetFontStyle("AdvancedRenamer.Style.RegularFont"))
						.MinValue(1)
						.MaxValue(9)
						.Value(this, &FAdvancedRenamerRemovePrefixSection::GetPrefixCharsValue)
						.OnValueChanged(this, &FAdvancedRenamerRemovePrefixSection::OnPrefixRemoveCharactersChanged)
					]
				]
			]
		];
}

void FAdvancedRenamerRemovePrefixSection::ResetToDefault()
{
	bRemoveOldPrefixSection = false;
	PrefixWidgetSwitcherIndex = 0;
	RemovePrefixCharsValue = 1;
	RemovePrefixSeparatorText = LOCTEXT("AR_PrefixSeparatorText", "_");
	RemovePrefixType = EAdvancedRenamerRemoveOldType::Separator;
}

ECheckBoxState FAdvancedRenamerRemovePrefixSection::IsRemoveOldPrefixChecked() const
{
	return bRemoveOldPrefixSection ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

bool FAdvancedRenamerRemovePrefixSection::IsRemoveOldPrefixEnabled() const
{
	return bRemoveOldPrefixSection;
}

void FAdvancedRenamerRemovePrefixSection::OnRemoveOldPrefixCheckBoxChanged(ECheckBoxState InNewState)
{
	bRemoveOldPrefixSection = InNewState == ECheckBoxState::Checked;
	MarkRenamerDirty();
}

FText FAdvancedRenamerRemovePrefixSection::GetPrefixSeparatorText() const
{
	return RemovePrefixSeparatorText;
}

uint8 FAdvancedRenamerRemovePrefixSection::GetPrefixCharsValue() const
{
	return RemovePrefixCharsValue;
}

EAdvancedRenamerRemoveOldType FAdvancedRenamerRemovePrefixSection::GetRemoveOldPrefixType() const
{
	return RemovePrefixType;
}

bool FAdvancedRenamerRemovePrefixSection::OnPrefixSeparatorVerifyTextChanged(const FText& InText, FText& OutErrorText) const
{
	if (InText.ToString().Len() > 1)
	{
		OutErrorText = LOCTEXT("SeparatorError", "Separators can only be a single character.");
		return false;
	}

	return true;
}

void FAdvancedRenamerRemovePrefixSection::OnPrefixSeparatorChanged(const FText& InNewText)
{
	RemovePrefixSeparatorText = InNewText;
	MarkRenamerDirty();
}

void FAdvancedRenamerRemovePrefixSection::OnPrefixRemoveCharactersChanged(uint8 InNewValue)
{
	RemovePrefixCharsValue = InNewValue;
	MarkRenamerDirty();
}

void FAdvancedRenamerRemovePrefixSection::OnRemoveOldPrefixTypeChanged(EAdvancedRenamerRemoveOldType InNewValue)
{
	RemovePrefixType = InNewValue;
	switch (InNewValue)
	{
	case EAdvancedRenamerRemoveOldType::Separator:
		PrefixWidgetSwitcherIndex = 0;
		break;
	case EAdvancedRenamerRemoveOldType::Chars:
		PrefixWidgetSwitcherIndex = 1;
		break;
	default:
		break;
	}
	MarkRenamerDirty();
}

bool FAdvancedRenamerRemovePrefixSection::CanApplyRemovePrefixSeparatorOperation()
{
	return RemovePrefixType == EAdvancedRenamerRemoveOldType::Separator && RemovePrefixSeparatorText.ToString().Len() == 1;
}

bool FAdvancedRenamerRemovePrefixSection::CanApplyRemovePrefixCharOperation()
{
	return RemovePrefixType == EAdvancedRenamerRemoveOldType::Chars;
}

void FAdvancedRenamerRemovePrefixSection::ApplyRemovePrefixSeparatorOperation(FString& OutOriginalName)
{
	FString Separator = RemovePrefixSeparatorText.ToString();
	const int32 PrefixStart = OutOriginalName.Find(Separator, ESearchCase::IgnoreCase);

	if (PrefixStart >= 0)
	{
		OutOriginalName.RightChopInline(PrefixStart + Separator.Len());
	}
}

void FAdvancedRenamerRemovePrefixSection::ApplyRemovePrefixCharOperation(FString& OutOriginalName)
{
	OutOriginalName.RightChopInline(RemovePrefixCharsValue);
}

void FAdvancedRenamerRemovePrefixSection::ApplyRemovePrefixOperation(FString& OutOriginalName)
{
	if (bRemoveOldPrefixSection)
	{
		if (CanApplyRemovePrefixSeparatorOperation())
		{
			ApplyRemovePrefixSeparatorOperation(OutOriginalName);
		}
		else if (CanApplyRemovePrefixCharOperation())
		{
			ApplyRemovePrefixCharOperation(OutOriginalName);
		}
	}
}

#undef LOCTEXT_NAMESPACE
