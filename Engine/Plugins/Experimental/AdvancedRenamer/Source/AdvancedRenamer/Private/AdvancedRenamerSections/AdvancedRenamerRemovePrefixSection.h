// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AdvancedRenamerSectionBase.h"
#include "Styling/SlateTypes.h"
#include "Utils/AdvancedRenamerSlateUtils.h"

class FString;
class SCheckBox;
class SEditableTextBox;
template <typename InNumericType>
class SSpinBox;
class SWidget;

class FAdvancedRenamerRemovePrefixSection : public FAdvancedRenamerSectionBase
{
public:
	FAdvancedRenamerRemovePrefixSection();

	virtual ~FAdvancedRenamerRemovePrefixSection() = default;

	/** Init the given section */
	virtual void Init(TSharedRef<IAdvancedRenamer> InRenamer) override;

	/** Return the widget for this section*/
	virtual TSharedRef<SWidget> GetWidget() override;

	/** Reset all values of the section to the default ones */
	virtual void ResetToDefault() override;

private:
	/** If checked the section will be enabled */
	ECheckBoxState IsRemoveOldPrefixChecked() const;

	/** If checked it will remove all the current suffix numbers */
	bool IsRemoveOldPrefixEnabled() const;

	/** Get the separator text */
	FText GetPrefixSeparatorText() const;

	/** Get the numbers of chars to remove */
	uint8 GetPrefixCharsValue() const;

	/** Get the type of the remove, either by Separator or by number of Chars */
	EAdvancedRenamerRemoveOldType GetRemoveOldPrefixType() const;

	/** Called when the RemoveOldPrefix checkbox state change */
	void OnRemoveOldPrefixCheckBoxChanged(ECheckBoxState InNewState);

	/** Verify that the separator text is valid */
	bool OnPrefixSeparatorVerifyTextChanged(const FText& InText, FText& OutErrorText) const;

	/** Called when the separator text change */
	void OnPrefixSeparatorChanged(const FText& InNewText);

	/** Called when the remove chars number change */
	void OnPrefixRemoveCharactersChanged(uint8 InNewValue);

	/** Called when the remove type changed, it will be either by Separator or by number of Chars */
	void OnRemoveOldPrefixTypeChanged(EAdvancedRenamerRemoveOldType InNewValue);

	/** Whether or not the Remove by Separator operation can be executed */
	bool CanApplyRemovePrefixSeparatorOperation();

	/** Whether or not the Remove by number of Chars operation can be executed */
	bool CanApplyRemovePrefixCharOperation();

	/** Execute logic for the Remove by Separator */
	void ApplyRemovePrefixSeparatorOperation(FString& OutOriginalName);

	/** Execute logic for the Remove by number of Chars */
	void ApplyRemovePrefixCharOperation(FString& OutOriginalName);

	/** Execute logic for this section */
	void ApplyRemovePrefixOperation(FString& OutOriginalName);

private:
	/** Section enabler CheckBox */
	TSharedPtr<SCheckBox> RemoveOldPrefixCheckBox;

	/** Separator EditableTextBox */
	TSharedPtr<SEditableTextBox> PrefixSeparatorTextBox;

	/** Number of Chars SSpinBox */
	TSharedPtr<SSpinBox<uint8>> PrefixRemoveCharactersSpinBox;
	
	/** Number of Chars value */
	uint8 RemovePrefixCharsValue;

	/** Separator Text */
	FText RemovePrefixSeparatorText;
	
	/** Index of the WidgetSwitcher */
	int32 PrefixWidgetSwitcherIndex;

	/** RemoveSuffix Remove type, either Separator or by number of Chars */
	EAdvancedRenamerRemoveOldType RemovePrefixType;

	/** RemoveOldSuffix section enabler, section is enabled if true */
	bool bRemoveOldPrefixSection;
};
