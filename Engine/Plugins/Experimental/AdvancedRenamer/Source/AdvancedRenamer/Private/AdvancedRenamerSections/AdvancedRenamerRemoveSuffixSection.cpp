// Copyright Epic Games, Inc. All Rights Reserved.

#include "AdvancedRenamerRemoveSuffixSection.h"

#include "IAdvancedRenamer.h"
#include "AdvancedRenamerStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "AdvancedRenamerRemoveSuffixSection"

FAdvancedRenamerRemoveSuffixSection::FAdvancedRenamerRemoveSuffixSection()
{
	FAdvancedRenamerRemoveSuffixSection::ResetToDefault();
}

void FAdvancedRenamerRemoveSuffixSection::Init(TSharedRef<IAdvancedRenamer> InRenamer)
{
	FAdvancedRenamerSectionBase::Init(InRenamer);
	Section.SectionName = TEXT("RemoveSuffix");
	Section.OnOperationExecuted().BindSP(this, &FAdvancedRenamerRemoveSuffixSection::ApplyRemoveSuffixOperation);
	InRenamer->AddSection(Section);
}

TSharedRef<SWidget> FAdvancedRenamerRemoveSuffixSection::GetWidget()
{
	using namespace AdvancedRenamerSlateUtils::Default;

	return SNew(SBorder)
		.BorderImage(FAdvancedRenamerStyle::Get().GetBrush("AdvancedRenamer.Style.BackgroundBorder"))
		.Content()
		[
			SNew(SVerticalBox)

			// Remove Suffix CheckBox
			+ SVerticalBox::Slot()
			.Padding(SectionContentFirstEntryPadding)
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(FirstWidgetPadding)
				[
					SAssignNew(RemoveOldSuffixCheckBox, SCheckBox)
					.IsChecked(this, &FAdvancedRenamerRemoveSuffixSection::IsRemoveOldSuffixChecked)
					.OnCheckStateChanged(this, &FAdvancedRenamerRemoveSuffixSection::OnRemoveOldSuffixCheckBoxChanged)
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font(FAdvancedRenamerStyle::Get().GetFontStyle("AdvancedRenamer.Style.RegularFont"))
					.Text(LOCTEXT("AR_RemoveOldSuffix", "Remove Old Suffix"))
				]
			]

			// Remove Suffix
			+ SVerticalBox::Slot()
			.Padding(SectionContentMiddleEntriesPadding)
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(FirstWidgetPadding)
				[
					SNew(SBox)
					.HeightOverride(25.f)
					.MinDesiredWidth(160.f)
					[
						SNew(SSegmentedControl<EAdvancedRenamerRemoveOldType>)
						.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
						.IsEnabled(this, &FAdvancedRenamerRemoveSuffixSection::IsRemoveOldSuffixEnabled)
						.UniformPadding(FMargin(2.f, 0.f))
						.SupportsEmptySelection(false)
						.SupportsMultiSelection(false)
						.Value(this, &FAdvancedRenamerRemoveSuffixSection::GetRemoveOldSuffixType)
						.OnValueChanged(this, &FAdvancedRenamerRemoveSuffixSection::OnRemoveOldSuffixTypeChanged)

						+ SSegmentedControl<EAdvancedRenamerRemoveOldType>::Slot(EAdvancedRenamerRemoveOldType::Separator)
						.Text(LOCTEXT("AR_SuffixSeparator", "Separator"))

						+ SSegmentedControl<EAdvancedRenamerRemoveOldType>::Slot(EAdvancedRenamerRemoveOldType::Chars)
						.Text(LOCTEXT("AR_SuffixFirstChar(s)", "Last Char(s)"))
					]
				]
				+ SHorizontalBox::Slot()
				.Padding(LastWidgetPadding)
				[
					SNew(SWidgetSwitcher)
					.IsEnabled(this, &FAdvancedRenamerRemoveSuffixSection::IsRemoveOldSuffixEnabled)
					.WidgetIndex(TAttribute<int32>::CreateLambda([this] () { return SuffixWidgetSwitcherIndex; }))

					+ SWidgetSwitcher::Slot()
					[
						SAssignNew(SuffixSeparatorTextBox, SEditableTextBox)
						.Font(FAdvancedRenamerStyle::Get().GetFontStyle("AdvancedRenamer.Style.RegularFont"))
						.Text(this, &FAdvancedRenamerRemoveSuffixSection::GetSuffixSeparatorText)
						.OnVerifyTextChanged(this, &FAdvancedRenamerRemoveSuffixSection::OnSuffixSeparatorVerifyTextChanged)
						.OnTextChanged(this, &FAdvancedRenamerRemoveSuffixSection::OnSuffixSeparatorChanged)
					]

					+ SWidgetSwitcher::Slot()
					[
						SAssignNew(SuffixRemoveCharactersSpinBox, SSpinBox<uint8>)
						.Style(&FAppStyle::Get(), "Menu.SpinBox")
						.Font(FAdvancedRenamerStyle::Get().GetFontStyle("AdvancedRenamer.Style.RegularFont"))
						.MinValue(1)
						.MaxValue(9)
						.Value(this, &FAdvancedRenamerRemoveSuffixSection::GetSuffixCharsValue)
						.OnValueChanged(this, &FAdvancedRenamerRemoveSuffixSection::OnSuffixRemoveCharactersChanged)
					]
				]
			]

			// Remove Suffix Numbers
			+ SVerticalBox::Slot()
			.Padding(SectionContentMiddleEntriesPadding)
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(FirstWidgetPadding)
				.AutoWidth()
				[
					SAssignNew(SuffixRemoveNumberCheckBox, SCheckBox)
					.IsChecked(this, &FAdvancedRenamerRemoveSuffixSection::IsSuffixRemoveNumberChecked)
					.OnCheckStateChanged(this, &FAdvancedRenamerRemoveSuffixSection::OnSuffixRemoveNumberCheckBoxChanged)
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font(FAdvancedRenamerStyle::Get().GetFontStyle("AdvancedRenamer.Style.RegularFont"))
					.Text(LOCTEXT("AR_SuffixRemoveNumber", "Remove Old Numbering"))
				]
			]
		];
}

void FAdvancedRenamerRemoveSuffixSection::ResetToDefault()
{
	bRemoveOldSuffixSection = false;
	bRemoveSuffixNumbers = false;
	SuffixWidgetSwitcherIndex = 0;
	RemoveSuffixCharsValue = 1;
	RemoveSuffixSeparatorText = LOCTEXT("AR_SuffixSeparatorText", "_");
	RemoveSuffixType = EAdvancedRenamerRemoveOldType::Separator;
}

ECheckBoxState FAdvancedRenamerRemoveSuffixSection::IsRemoveOldSuffixChecked() const
{
	return bRemoveOldSuffixSection ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

ECheckBoxState FAdvancedRenamerRemoveSuffixSection::IsSuffixRemoveNumberChecked() const
{
	return bRemoveSuffixNumbers ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

bool FAdvancedRenamerRemoveSuffixSection::IsRemoveOldSuffixEnabled() const
{
	return bRemoveOldSuffixSection;
}

void FAdvancedRenamerRemoveSuffixSection::OnRemoveOldSuffixCheckBoxChanged(ECheckBoxState InNewState)
{
	bRemoveOldSuffixSection = InNewState == ECheckBoxState::Checked;
	MarkRenamerDirty();
}

void FAdvancedRenamerRemoveSuffixSection::OnSuffixRemoveNumberCheckBoxChanged(ECheckBoxState InNewState)
{
	bRemoveSuffixNumbers = InNewState == ECheckBoxState::Checked;
	MarkRenamerDirty();
}

FText FAdvancedRenamerRemoveSuffixSection::GetSuffixSeparatorText() const
{
	return RemoveSuffixSeparatorText;
}

uint8 FAdvancedRenamerRemoveSuffixSection::GetSuffixCharsValue() const
{
	return RemoveSuffixCharsValue;
}

EAdvancedRenamerRemoveOldType FAdvancedRenamerRemoveSuffixSection::GetRemoveOldSuffixType() const
{
	return RemoveSuffixType;
}

bool FAdvancedRenamerRemoveSuffixSection::OnSuffixSeparatorVerifyTextChanged(const FText& InText, FText& OutErrorText) const
{
	if (InText.ToString().Len() > 1)
	{
		OutErrorText = LOCTEXT("SeparatorError", "Separators can only be a single character.");
		return false;
	}

	return true;
}

void FAdvancedRenamerRemoveSuffixSection::OnSuffixSeparatorChanged(const FText& InNewText)
{
	RemoveSuffixSeparatorText = InNewText;
	MarkRenamerDirty();
}

void FAdvancedRenamerRemoveSuffixSection::OnSuffixRemoveCharactersChanged(uint8 InNewValue)
{
	RemoveSuffixCharsValue = InNewValue;
	MarkRenamerDirty();
}

void FAdvancedRenamerRemoveSuffixSection::OnRemoveOldSuffixTypeChanged(EAdvancedRenamerRemoveOldType InNewValue)
{
	RemoveSuffixType = InNewValue;
	switch (InNewValue)
	{
		case EAdvancedRenamerRemoveOldType::Separator:
			SuffixWidgetSwitcherIndex = 0;
			break;
		case EAdvancedRenamerRemoveOldType::Chars:
			SuffixWidgetSwitcherIndex = 1;
			break;
		default:
			break;
	}
	MarkRenamerDirty();
}

bool FAdvancedRenamerRemoveSuffixSection::CanApplyRemoveSuffixSeparatorOperation()
{
	return RemoveSuffixType == EAdvancedRenamerRemoveOldType::Separator && RemoveSuffixSeparatorText.ToString().Len() == 1;
}

bool FAdvancedRenamerRemoveSuffixSection::CanApplyRemoveSuffixCharOperation()
{
	return RemoveSuffixType == EAdvancedRenamerRemoveOldType::Chars;
}

bool FAdvancedRenamerRemoveSuffixSection::CanApplyRemoveSuffixNumbers()
{
	return bRemoveSuffixNumbers;
}

void FAdvancedRenamerRemoveSuffixSection::ApplyRemoveSuffixSeparatorOperation(FString& OutOriginalName)
{
	FString Separator = RemoveSuffixSeparatorText.ToString();
	const int32 SuffixStart = OutOriginalName.Find(Separator, ESearchCase::IgnoreCase, ESearchDir::FromEnd);

	if (SuffixStart >= 0)
	{
		OutOriginalName.LeftInline(SuffixStart);
	}
}

void FAdvancedRenamerRemoveSuffixSection::ApplyRemoveSuffixCharOperation(FString& OutOriginalName)
{
	OutOriginalName.LeftChopInline(RemoveSuffixCharsValue);
}

void FAdvancedRenamerRemoveSuffixSection::ApplyRemoveSuffixNumbers(FString& OutOriginalName)
{
	static constexpr TCHAR FirstDigit = '0';
	static constexpr TCHAR LastDigit = '9';

	int32 LastDigitIndex = OutOriginalName.Len() - 1;

	while (LastDigitIndex >= 0)
	{
		if (OutOriginalName[LastDigitIndex] < FirstDigit || OutOriginalName[LastDigitIndex] > LastDigit)
		{
			break;
		}

		--LastDigitIndex;
	}

	OutOriginalName.LeftInline(LastDigitIndex + 1);
}

void FAdvancedRenamerRemoveSuffixSection::ApplyRemoveSuffixOperation(FString& OutOriginalName)
{
	if (CanApplyRemoveSuffixNumbers())
	{
		ApplyRemoveSuffixNumbers(OutOriginalName);
	}

	if (bRemoveOldSuffixSection)
	{
		if (CanApplyRemoveSuffixSeparatorOperation())
		{
			ApplyRemoveSuffixSeparatorOperation(OutOriginalName);
		}
		else if (CanApplyRemoveSuffixCharOperation())
		{
			ApplyRemoveSuffixCharOperation(OutOriginalName);
		}
	}
}

#undef LOCTEXT_NAMESPACE
