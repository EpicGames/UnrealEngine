// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AdvancedRenamerSectionBase.h"
#include "Styling/SlateTypes.h"
#include "Utils/AdvancedRenamerSlateUtils.h"

class FString;
class SCheckBox;
class SEditableTextBox;
template <typename InNumericType>
class SSpinBox;
class SWidget;

/**
 * RemoveSuffix section
 */
class FAdvancedRenamerRemoveSuffixSection : public FAdvancedRenamerSectionBase
{
public:
	FAdvancedRenamerRemoveSuffixSection();

	virtual ~FAdvancedRenamerRemoveSuffixSection() = default;

	/** Init the given section */
	virtual void Init(TSharedRef<IAdvancedRenamer> InRenamer) override;

	/** Return the widget for this section*/
	virtual TSharedRef<SWidget> GetWidget() override;

	/** Reset all values of the section to the default ones */
	virtual void ResetToDefault() override;

private:

	/** If checked the section will be enabled */
	ECheckBoxState IsRemoveOldSuffixChecked() const;

	/** If checked it will remove all the current suffix numbers */
	ECheckBoxState IsSuffixRemoveNumberChecked() const;

	/** Whether or not the section is enabled */
	bool IsRemoveOldSuffixEnabled() const;

	/** Get the separator text */
	FText GetSuffixSeparatorText() const;

	/** Get the numbers of chars to remove */
	uint8 GetSuffixCharsValue() const;

	/** Get the type of the remove, either by Separator or by number of Chars */
	EAdvancedRenamerRemoveOldType GetRemoveOldSuffixType() const;

	/** Called when the RemoveOldSuffix checkbox state change */
	void OnRemoveOldSuffixCheckBoxChanged(ECheckBoxState InNewState);

	/** Called when the RemoveNumber checkbox state change */
	void OnSuffixRemoveNumberCheckBoxChanged(ECheckBoxState InNewState);

	/** Verify that the separator text is valid */
	bool OnSuffixSeparatorVerifyTextChanged(const FText& InText, FText& OutErrorText) const;

	/** Called when the separator text change */
	void OnSuffixSeparatorChanged(const FText& InNewText);

	/** Called when the remove chars number change */
	void OnSuffixRemoveCharactersChanged(uint8 InNewValue);

	/** Called when the remove type changed, it will be either by Separator or by number of Chars */
	void OnRemoveOldSuffixTypeChanged(EAdvancedRenamerRemoveOldType InNewValue);

	/** Whether or not the Remove by Separator operation can be executed */
	bool CanApplyRemoveSuffixSeparatorOperation();

	/** Whether or not the Remove by number of Chars operation can be executed */
	bool CanApplyRemoveSuffixCharOperation();

	/** Whether or not the RemoveNumbers operation can be executed */
	bool CanApplyRemoveSuffixNumbers();

	/** Execute logic for the Remove by Separator */
	void ApplyRemoveSuffixSeparatorOperation(FString& OutOriginalName);

	/** Execute logic for the Remove by number of Chars */
	void ApplyRemoveSuffixCharOperation(FString& OutOriginalName);

	/** Execute logic for the RemoveNumbers */
	void ApplyRemoveSuffixNumbers(FString& OutOriginalName);

	/** Execute logic for this section */
	void ApplyRemoveSuffixOperation(FString& OutOriginalName);

private:
	/** Section enabler CheckBox */
	TSharedPtr<SCheckBox> RemoveOldSuffixCheckBox;
	
	/** Separator EditableTextBox */
	TSharedPtr<SEditableTextBox> SuffixSeparatorTextBox;

	/** Number of Chars SSpinBox */
	TSharedPtr<SSpinBox<uint8>> SuffixRemoveCharactersSpinBox;

	/** RemoveNumber CheckBox */
	TSharedPtr<SCheckBox> SuffixRemoveNumberCheckBox;

	/** Number of Chars value */
	uint8 RemoveSuffixCharsValue;

	/** Separator Text */
	FText RemoveSuffixSeparatorText;

	/** Index of the WidgetSwitcher */
	int32 SuffixWidgetSwitcherIndex;

	/** RemoveSuffix Remove type, either Separator or by number of Chars */
	EAdvancedRenamerRemoveOldType RemoveSuffixType;

	/** RemoveOldSuffix section enabler, section is enabled if true */
	bool bRemoveOldSuffixSection;

	/** RemoveNumber enabler, RemoveNumber operation will be executed if true */
	bool bRemoveSuffixNumbers;
};
