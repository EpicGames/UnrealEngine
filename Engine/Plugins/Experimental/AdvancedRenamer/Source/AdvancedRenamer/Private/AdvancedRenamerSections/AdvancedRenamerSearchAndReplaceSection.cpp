// Copyright Epic Games, Inc. All Rights Reserved.

#include "AdvancedRenamerSearchAndReplaceSection.h"

#include "AdvancedRenamerModule.h"
#include "AdvancedRenamerStyle.h"
#include "IAdvancedRenamer.h"
#include "Internationalization/Regex.h"
#include "Styling/StyleColors.h"
#include "Utils/AdvancedRenamerSlateUtils.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "AdvancedRenamerSearchAndReplaceSection"

FAdvancedRenamerSearchAndReplaceSection::FAdvancedRenamerSearchAndReplaceSection()
{
	FAdvancedRenamerSearchAndReplaceSection::ResetToDefault();
}

void FAdvancedRenamerSearchAndReplaceSection::Init(TSharedRef<IAdvancedRenamer> InRenamer)
{
	FAdvancedRenamerSectionBase::Init(InRenamer);
	Section.SectionName = TEXT("SearchAndReplaceSection");
	Section.OnOperationExecuted().BindSP(this, &FAdvancedRenamerSearchAndReplaceSection::ApplySearchAndReplaceOrRenameOperation);
	InRenamer->AddSection(Section);
}

TSharedRef<SWidget> FAdvancedRenamerSearchAndReplaceSection::GetWidget()
{
	using namespace AdvancedRenamerSlateUtils::Default;

	return SNew(SBorder)
		.BorderImage(FAdvancedRenamerStyle::Get().GetBrush("AdvancedRenamer.Style.BackgroundBorder"))
		.Content()
		[
			SNew(SVerticalBox)

			// Title
			+ SVerticalBox::Slot()
			.Padding(SectionContentFirstEntryPadding)
			.AutoHeight()
			[
				SNew(STextBlock)
				.Font(FAdvancedRenamerStyle::Get().GetFontStyle("AdvancedRenamer.Style.TitleFont"))
				.Text(LOCTEXT("AR_SearchReplaceTitle", "Rename"))
			]

			+ SVerticalBox::Slot()
			.Padding(SectionContentMiddleEntriesPadding)
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(FirstWidgetPadding)
				[
					SNew(SBox)
					.HeightOverride(25.f)
					.MinDesiredWidth(160.f)
					[
						SNew(SSegmentedControl<EAdvancedRenamerSearchAndReplaceType>)
						.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
						.UniformPadding(FMargin(2.f, 0.f))
						.SupportsEmptySelection(false)
						.SupportsMultiSelection(false)
						.Value(SearchAndReplaceType)
						.OnValueChanged_Lambda([this] (EAdvancedRenamerSearchAndReplaceType InSearchAndReplaceType)
						{
							SearchAndReplaceType = InSearchAndReplaceType;
							MarkRenamerDirty();
						})

						+ SSegmentedControl<EAdvancedRenamerSearchAndReplaceType>::Slot(EAdvancedRenamerSearchAndReplaceType::PlainText)
						.Text(LOCTEXT("BR_PlainText", "Plain Text"))

						+ SSegmentedControl<EAdvancedRenamerSearchAndReplaceType>::Slot(EAdvancedRenamerSearchAndReplaceType::RegularExpression)
						.Text(LOCTEXT("BR_Regex", "Regex"))
					]
				]

				// Regex Help
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(4.f, 0.f))
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::GetBrush("Icons.Help"))
					.OnMouseButtonDown(this, &FAdvancedRenamerSearchAndReplaceSection::OnRegexHelp)
				]

				// Ignore case
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(FMargin(4.f, 0.f))
				[
					SAssignNew(SearchReplaceIgnoreCaseCheckBox, SCheckBox)
					.IsChecked(this, &FAdvancedRenamerSearchAndReplaceSection::IsSearchReplaceIgnoreCaseChecked)
					.OnCheckStateChanged(this, &FAdvancedRenamerSearchAndReplaceSection::OnSearchReplaceIgnoreCaseCheckBoxChanged)
				]

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font(FAdvancedRenamerStyle::Get().GetFontStyle("AdvancedRenamer.Style.RegularFont"))
					.Text(LOCTEXT("AR_IgnoreCase", "Ignore Case"))
				]
			]

			// Search text
			+ SVerticalBox::Slot()
			.Padding(SectionContentMiddleEntriesPadding)
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.Padding(FirstWidgetPadding)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SBox)
					.WidthOverride(65.f)
					[
						SNew(STextBlock)
						.Font(FAdvancedRenamerStyle::Get().GetFontStyle("AdvancedRenamer.Style.RegularFont"))
						.Text(LOCTEXT("AR_SearchLabel", "Search"))
					]
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SAssignNew(SearchReplaceSearchTextBox, SEditableTextBox)
					.Font(FAdvancedRenamerStyle::Get().GetFontStyle("AdvancedRenamer.Style.RegularFont"))
					.HintText(LOCTEXT("AR_RegexSearchHint", "Search (Optional)"))
					.Text(this, &FAdvancedRenamerSearchAndReplaceSection::GetSearchText)
					.OnTextChanged(this, &FAdvancedRenamerSearchAndReplaceSection::OnSearchTextChanged)
				]
			]

			// Rename/Replace text
			+ SVerticalBox::Slot()
			.Padding(SectionContentMiddleEntriesPadding)
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.Padding(FirstWidgetPadding)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Font(FAdvancedRenamerStyle::Get().GetFontStyle("AdvancedRenamer.Style.RegularFont"))
					.Text(LOCTEXT("AR_RenameReplaceLabel", "Rename To"))
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SAssignNew(SearchReplaceReplaceTextBox, SEditableTextBox)
					.Font(FAdvancedRenamerStyle::Get().GetFontStyle("AdvancedRenamer.Style.RegularFont"))
					.HintText(LOCTEXT("AR_RenameReplaceHint", "Replace"))
					.Text(this, &FAdvancedRenamerSearchAndReplaceSection::GetReplaceText)
					.OnTextChanged(this, &FAdvancedRenamerSearchAndReplaceSection::OnReplaceTextChanged)
				]
			]
		];
}

void FAdvancedRenamerSearchAndReplaceSection::ResetToDefault()
{
	SearchCaseType = ESearchCase::IgnoreCase;
	SearchAndReplaceType = EAdvancedRenamerSearchAndReplaceType::PlainText;
	SearchText = FText::GetEmpty();
	ReplaceText = FText::GetEmpty();
}

FReply FAdvancedRenamerSearchAndReplaceSection::OnRegexHelp(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	FPlatformProcess::LaunchURL(*GetRegexDocumentationURL(), nullptr, nullptr);
	return FReply::Handled();
}

ECheckBoxState FAdvancedRenamerSearchAndReplaceSection::IsSearchReplaceIgnoreCaseChecked() const
{
	return SearchCaseType == ESearchCase::IgnoreCase ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FAdvancedRenamerSearchAndReplaceSection::OnSearchReplaceIgnoreCaseCheckBoxChanged(ECheckBoxState InNewState)
{
	SearchCaseType = InNewState == ECheckBoxState::Checked ? ESearchCase::IgnoreCase : ESearchCase::CaseSensitive;
	if (const TSharedPtr<IAdvancedRenamer>& Renamer = RenamerWeakPtr.Pin())
	{
		Renamer->MarkDirty();
	}
}

void FAdvancedRenamerSearchAndReplaceSection::OnSearchTextChanged(const FText& InNewText)
{
	SearchText = InNewText;
	MarkRenamerDirty();
}

FText FAdvancedRenamerSearchAndReplaceSection::GetSearchText() const
{
	return SearchText;
}

void FAdvancedRenamerSearchAndReplaceSection::OnReplaceTextChanged(const FText& InNewText)
{
	ReplaceText = InNewText;
	MarkRenamerDirty();
}

FText FAdvancedRenamerSearchAndReplaceSection::GetReplaceText() const
{
	return ReplaceText;
}

FString FAdvancedRenamerSearchAndReplaceSection::RegexReplace(const FString& InOriginalString, const FRegexPattern& InPattern, const FString& InReplaceString) const
{
	static const FString EscapeString = TEXT("\\");
	static constexpr TCHAR EscapeChar = '\\';
	static const FString GroupString = TEXT("$");
	static constexpr TCHAR GroupChar = '$';
	static constexpr TCHAR FirstDigit = '0';
	static constexpr TCHAR LastDigit = '9';
	static constexpr TCHAR NullChar = 0;

	FRegexMatcher Matcher(InPattern, InOriginalString);

	FString Output = "";
	int32 StartCharIndex = 0;
	bool bReadingGroupName = false;
	int32 GroupIndex = INDEX_NONE;

	while (Matcher.FindNext())
	{
		// Add on part of string after start/previous match
		if (StartCharIndex != Matcher.GetMatchBeginning())
		{
			Output += InOriginalString.Mid(StartCharIndex, Matcher.GetMatchBeginning() - StartCharIndex);
		}

		bool bEscaped = false;

		for (int32 CharIndex = 0; CharIndex <= InReplaceString.Len(); ++CharIndex)
		{
			const TCHAR& Char = CharIndex < InReplaceString.Len() ? InReplaceString[CharIndex] : NullChar;

			if (bReadingGroupName)
			{
				// Build group index
				if (Char >= FirstDigit && Char <= LastDigit)
				{
					if (GroupIndex == INDEX_NONE)
					{
						GroupIndex = 0;
					}
					else if (GroupIndex > 0)
					{
						GroupIndex *= 10;
					}

					const int32 NextDigit = static_cast<int32>(Char - FirstDigit);
					GroupIndex += NextDigit;
					continue;
				}
				// We've read a group index, add it to the output string
				else if (GroupIndex > 0)
				{
					if (Matcher.GetCaptureGroupBeginning(GroupIndex) == INDEX_NONE)
					{
						UE_LOGF(LogARP, Error, "Regex: Capture group does not exist %d.", GroupIndex);
					}

					Output += Matcher.GetCaptureGroup(GroupIndex);
				}
				// $0 matches the entire matched string
				else if (GroupIndex == 0)
				{
					Output += InOriginalString.Mid(Matcher.GetMatchBeginning(), Matcher.GetMatchEnding() - Matcher.GetMatchBeginning());
				}
				// An unescaped $
				else
				{
					UE_LOGF(LogARP, Error, "Regex: Unescaped %ls.", *GroupString);

					Output += GroupString;
				}

				bReadingGroupName = false;
				// Continue regular parsing of this character.
			}

			// Check for special chars
			if (!bEscaped)
			{
				if (Char == EscapeChar)
				{
					bEscaped = true;
					continue;
				}

				if (Char == GroupChar)
				{
					bReadingGroupName = true;
					GroupIndex = INDEX_NONE;
					continue;
				}
			}
			else
			{
				// If the last char is a \ assume that it's not an escape char
				if (Char == NullChar)
				{
					UE_LOGF(LogARP, Error, "Regex: Unescaped %ls.", *EscapeString);

					Output += EscapeChar;
				}
			}

			if (Char != NullChar)
			{
				Output += Char;
			}

			bEscaped = false;
		}

		StartCharIndex = Matcher.GetMatchEnding();
	}

	// Add on the end of the string after the last match
	if (StartCharIndex < InOriginalString.Len())
	{
		Output += InOriginalString.Mid(StartCharIndex);
	}

	return Output;
}

bool FAdvancedRenamerSearchAndReplaceSection::CanApplySearchAndReplaceOperation()
{
	return !SearchText.IsEmpty();
}

bool FAdvancedRenamerSearchAndReplaceSection::CanApplyRenameOperation()
{
	return !ReplaceText.IsEmpty() && SearchText.IsEmpty();
}

void FAdvancedRenamerSearchAndReplaceSection::ApplySearchAndReplaceOperation(FString& OutOriginalName)
{
	if (SearchAndReplaceType == EAdvancedRenamerSearchAndReplaceType::PlainText)
	{
		OutOriginalName = OutOriginalName.Replace(*SearchText.ToString(), *ReplaceText.ToString(), SearchCaseType);
	}
	else
	{
		const FRegexPattern RegexPattern = FRegexPattern(SearchText.ToString(),
			SearchCaseType == ESearchCase::IgnoreCase ? ERegexPatternFlags::CaseInsensitive : ERegexPatternFlags::None);

		OutOriginalName = RegexReplace(OutOriginalName, RegexPattern, ReplaceText.ToString());
	}
}

void FAdvancedRenamerSearchAndReplaceSection::ApplyRenameOperation(FString& OutOriginalName)
{
	OutOriginalName = ReplaceText.ToString();
}

void FAdvancedRenamerSearchAndReplaceSection::ApplySearchAndReplaceOrRenameOperation(FString& OutOriginalName)
{
	if (CanApplySearchAndReplaceOperation())
	{
		ApplySearchAndReplaceOperation(OutOriginalName);
	}
	else if (CanApplyRenameOperation())
	{
		ApplyRenameOperation(OutOriginalName);
	}
}

#undef LOCTEXT_NAMESPACE
