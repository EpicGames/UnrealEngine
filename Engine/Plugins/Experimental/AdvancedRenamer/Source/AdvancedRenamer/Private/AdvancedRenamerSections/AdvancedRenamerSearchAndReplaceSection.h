// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AdvancedRenamerSectionBase.h"
#include "Input/Reply.h"
#include "Styling/SlateTypes.h"

class FRegexPattern;
class SCheckBox;
class SEditableTextBox;
struct FGeometry;
struct FPointerEvent;

/**
 * Search And Replace options
 */
enum class EAdvancedRenamerSearchAndReplaceType
{
	PlainText,
	RegularExpression
};

/**
 * Search and Replace/Rename section
 */
class FAdvancedRenamerSearchAndReplaceSection : public FAdvancedRenamerSectionBase
{
public:
	FAdvancedRenamerSearchAndReplaceSection();

	virtual ~FAdvancedRenamerSearchAndReplaceSection() = default;

	/** Init the given section */
	virtual void Init(TSharedRef<IAdvancedRenamer> InRenamer) override;

	/** Return the widget for this section*/
	virtual TSharedRef<SWidget> GetWidget() override;

	/** Reset all values of the section to the default ones */
	virtual void ResetToDefault() override;

private:
	/** Regex documentation URL */
	static FString GetRegexDocumentationURL() { return TEXT("https://unicode-org.github.io/icu/userguide/strings/regexp.html"); }

	/** Called when the Regex help icon is clicked */
	FReply OnRegexHelp(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent);

	/** If checked it will use the IgnoreCase, otherwise it is CaseSensitive */
	ECheckBoxState IsSearchReplaceIgnoreCaseChecked() const;

	/** Called when the ignore case checkbox state change */
	void OnSearchReplaceIgnoreCaseCheckBoxChanged(ECheckBoxState InNewState);

	/** Called when the search text change */
	void OnSearchTextChanged(const FText& InNewText);

	/** Get the search text */
	FText GetSearchText() const;

	/** Called when the replace text change */
	void OnReplaceTextChanged(const FText& InNewText);

	/** Get the replace text */
	FText GetReplaceText() const;

	/** Regex Replace logic */
	FString RegexReplace(const FString& InOriginalString, const FRegexPattern& InPattern, const FString& InReplaceString) const;

	/** Whether or not the SearchAndReplace operation can be executed */
	bool CanApplySearchAndReplaceOperation();

	/** Whether or not the Rename operation can be executed */
	bool CanApplyRenameOperation();

	/** Execute logic for the SearchAndReplace */
	void ApplySearchAndReplaceOperation(FString& OutOriginalName);

	/** Execute logic for the Rename */
	void ApplyRenameOperation(FString& OutOriginalName);

	/** Execute logic for this section */
	void ApplySearchAndReplaceOrRenameOperation(FString& OutOriginalName);

private:
	/** Search EditableTextBox */
	TSharedPtr<SEditableTextBox> SearchReplaceSearchTextBox;

	/** Replace/Rename EditableTextBox */
	TSharedPtr<SEditableTextBox> SearchReplaceReplaceTextBox;

	/** Ignore case CheckBox */
	TSharedPtr<SCheckBox> SearchReplaceIgnoreCaseCheckBox;

	/** SearchAndReplace search type, either PlainText or RegularExpression */
	EAdvancedRenamerSearchAndReplaceType SearchAndReplaceType;

	/** SearchAndReplace case type, either IgnoreCase or CaseSensitive */
	ESearchCase::Type SearchCaseType;

	/** Search Text */
	FText SearchText;

	/** Replace/Rename Text */
	FText ReplaceText;
};
