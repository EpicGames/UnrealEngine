// Copyright Epic Games, Inc. All Rights Reserved.

#include "AdvancedRenamerSectionBase.h"

#include "IAdvancedRenamer.h"

void FAdvancedRenamerSectionBase::Init(TSharedRef<IAdvancedRenamer> InRenamer)
{
	RenamerWeakPtr = InRenamer;
	ResetToDefault();
}

FAdvancedRenamerExecuteSection FAdvancedRenamerSectionBase::GetSection() const
{
	return Section;
}

void FAdvancedRenamerSectionBase::MarkRenamerDirty() const
{
	if (const TSharedPtr<IAdvancedRenamer>& Renamer = RenamerWeakPtr.Pin())
	{
		Renamer->MarkDirty();
	}
}
