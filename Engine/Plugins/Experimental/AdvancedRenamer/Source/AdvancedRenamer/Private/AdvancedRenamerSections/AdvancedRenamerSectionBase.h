// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAdvancedRenamerSection.h"

/**
 * Base class for all the default sections, implementing common behavior for them
 */
class FAdvancedRenamerSectionBase : public IAdvancedRenamerSection
{
public:
	/** Init the given section */
	virtual void Init(TSharedRef<IAdvancedRenamer> InRenamer) override;

	/** Return the section information*/
	virtual FAdvancedRenamerExecuteSection GetSection() const override;

protected:
	/** Mark the Renamer dirty, this will make the Renamer re-execute the Rename logic */
	virtual void MarkRenamerDirty() const override;

protected:
	/** Section information of this extension */
	FAdvancedRenamerExecuteSection Section;
	
	/** Weak pointer to the Renamer */
	TWeakPtr<IAdvancedRenamer> RenamerWeakPtr;
};
