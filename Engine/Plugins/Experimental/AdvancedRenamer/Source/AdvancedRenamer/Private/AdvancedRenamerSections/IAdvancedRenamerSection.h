// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AdvancedRenamerExecuteSection.h"
#include "Templates/SharedPointerFwd.h"

class IAdvancedRenamer;
class SWidget;

/**
 * Interface for all the sections for the AdvancedRenamer
 */
class IAdvancedRenamerSection : public TSharedFromThis<IAdvancedRenamerSection>
{
public:
	/**
	 * Create a new AdvancedRenamer section given its constructor params
	 * @tparam InAdvancedRenamerSectionFactory AdvancedRenamer section class
	 * @tparam InArgsType constructor parameter type
	 * @param InArgs Additional constructor parameter
	 * @return The newly created AdvancedRenamer section
	 */
	template <
		typename InAdvancedRenamerSectionFactory,
		typename... InArgsType
		UE_REQUIRES(TIsDerivedFrom<InAdvancedRenamerSectionFactory, IAdvancedRenamerSection>::Value)
	>
	static TSharedRef<InAdvancedRenamerSectionFactory> MakeInstance(InArgsType&&... InArgs)
	{
		return MakeShared<InAdvancedRenamerSectionFactory>(Forward<InArgsType>(InArgs)...);
	}

	/** virtual destructor */
	virtual ~IAdvancedRenamerSection() = default;

	/** Init the given section */
	virtual void Init(TSharedRef<IAdvancedRenamer> InRenamer) = 0;

	/** Reset all values of the section to the default ones */
	virtual void ResetToDefault() = 0;

	/** Return the section information*/
	virtual FAdvancedRenamerExecuteSection GetSection() const = 0;

	/** Return the widget for this section*/
	virtual TSharedRef<SWidget> GetWidget() = 0;

protected:
	/** Mark the Renamer dirty, this will make the Renamer re-execute the Rename logic */
	virtual void MarkRenamerDirty() const = 0;
};
