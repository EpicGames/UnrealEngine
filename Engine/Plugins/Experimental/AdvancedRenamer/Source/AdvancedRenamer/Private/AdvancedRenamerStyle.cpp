// Copyright Epic Games, Inc. All Rights Reserved.

#include "AdvancedRenamerStyle.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"

#define IMAGE_BRUSH(RelativePath, ...) FSlateImageBrush(StyleInstance->RootToContentDir(RelativePath, TEXT(".png") ), __VA_ARGS__)

TSharedPtr<FSlateStyleSet> FAdvancedRenamerStyle::StyleInstance = nullptr;

void FAdvancedRenamerStyle::Initialize()
{
	if (!StyleInstance.IsValid())
	{
		InitStyle();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}

void FAdvancedRenamerStyle::Shutdown()
{
	if (StyleInstance.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
		ensure(StyleInstance.IsUnique());
		StyleInstance.Reset();
	}
}

FName FAdvancedRenamerStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("AdvancedRenamerStyle"));
	return StyleSetName;
}

void FAdvancedRenamerStyle::InitStyle()
{
	if (StyleInstance.IsValid())
	{
		return;
	}

	StyleInstance = MakeShared<FSlateStyleSet>("AdvancedRenamer");

	// Same ContentDir and CoreRootContentDir as the ContentBrowser
	StyleInstance->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	StyleInstance->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	const FSplitterStyle SplitterStyle = FSplitterStyle()
		.SetHandleNormalBrush(FSlateNoResource())
		.SetHandleHighlightBrush(FSlateNoResource());
	
	StyleInstance->Set("AdvancedRenamer.Style.Splitter", SplitterStyle);

	FSlateBrush* BackgroundBorderBrush = new FSlateColorBrush(FLinearColor::FromSRGBColor(FColor(36, 36, 36)));

	StyleInstance->Set("AdvancedRenamer.Style.BackgroundBorder", BackgroundBorderBrush);

	const FTableViewStyle ListViewStyle = FTableViewStyle()
		.SetBackgroundBrush(*FAdvancedRenamerStyle::Get().GetBrush("AdvancedRenamer.Style.BackgroundBorder"));

	StyleInstance->Set("AdvancedRenamer.Style.ListView", ListViewStyle);

	FHeaderRowStyle HeaderRowStyle = FAppStyle::Get().GetWidgetStyle<FHeaderRowStyle>("TableView.Header");
	HeaderRowStyle.SetHorizontalSeparatorThickness(0);
	HeaderRowStyle.SetHorizontalSeparatorBrush(FSlateNoResource());
	HeaderRowStyle.SetBackgroundBrush(FSlateColorBrush(FLinearColor::FromSRGBColor(FColor(47, 47, 47))));

	StyleInstance->Set("AdvancedRenamer.Style.HeaderRow", HeaderRowStyle);

	StyleInstance->Set("AdvancedRenamer.Style.TitleFont", FCoreStyle::GetDefaultFontStyle("Regular", 12));
	StyleInstance->Set("AdvancedRenamer.Style.RegularFont", FCoreStyle::GetDefaultFontStyle("Regular", 10));

	// Commands Icon
	StyleInstance->Set("AdvancedRenamer.BatchRenameObject", new IMAGE_BRUSH("Icons/Icon_Asset_Rename_16x", FVector2D(16.f, 16.f)));
	StyleInstance->Set("AdvancedRenamer.BatchRenameSharedClassActors", new IMAGE_BRUSH("Icons/Icon_Asset_Rename_16x", FVector2D(16.f, 16.f)));
}

const ISlateStyle& FAdvancedRenamerStyle::Get()
{
	if (!StyleInstance.IsValid())
	{
		Initialize();
	}

	return *StyleInstance;
}

#undef IMAGE_BRUSH
