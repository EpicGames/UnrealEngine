// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class FSlateStyleSet;
class ISlateStyle;
class FName;

class FAdvancedRenamerStyle
{
public:

	static void Initialize();

	static void Shutdown();

	static const ISlateStyle& Get();

	static FName GetStyleSetName();

private:

	static void InitStyle();
	static TSharedPtr<FSlateStyleSet> StyleInstance;

};
