// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AdvancedRenamerExecuteSection.h"
#include "Containers/UnrealString.h"
#include "Providers/IAdvancedRenamerProvider.h"
#include "Templates/SharedPointerFwd.h"

struct FAdvancedRenamerPreview
{
	FAdvancedRenamerPreview(int32 InHash, const FString InOriginalName)
		: Hash(InHash)
		, OriginalName(InOriginalName)
		, NewName(TEXT(""))
		, OriginalNameForSort(InOriginalName)
	{
	}

	/** Get the OriginalName FName for sort purposes */
	FName GetNameForSort() const { return OriginalNameForSort; }
	
public:
	int32 Hash;
	const FString OriginalName;
	FString NewName;

private:
	const FName OriginalNameForSort;
};

/**
* Implements its own provider interface so it can avoid long Execute_ lines and handle
* the 2 different types of provider (SharedPtr and UObject.)
*/
class IAdvancedRenamer : public IAdvancedRenamerProvider
{
public:
	virtual ~IAdvancedRenamer() = default;

	virtual const TSharedRef<IAdvancedRenamerProvider>& GetProvider() const = 0;

	/** Get the sortable previews */
	virtual TArray<TSharedPtr<FAdvancedRenamerPreview>>& GetSortablePreviews() = 0;

	/** Reset the order of the Previews to the original one */
	virtual void ResetSortablePreviews() = 0;

	/** Add a section to this Renamer */
	virtual void AddSection(FAdvancedRenamerExecuteSection InSection) = 0;

	/** True if there are any items actually renamed by the preview generator. */
	virtual bool HasRenames() const = 0;

	/** Whether the options have been updated. */
	virtual bool IsDirty() const = 0;
	virtual void MarkDirty() = 0;
	virtual void MarkClean() = 0;

	/** Executes the rename on the given name. */
	virtual FString ApplyRename(const FString& InName) = 0;
	
	/** Returns true if any names actually changed. */
	virtual bool UpdatePreviews() = 0;

	/** Returns true if all items were updated without error. */
	virtual bool Execute() = 0;
};
