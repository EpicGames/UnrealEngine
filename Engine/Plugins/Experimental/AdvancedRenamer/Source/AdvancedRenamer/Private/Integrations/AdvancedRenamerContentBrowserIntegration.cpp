// Copyright Epic Games, Inc. All Rights Reserved.

#include "Integrations/AdvancedRenamerContentBrowserIntegration.h"

#include "AdvancedRenamerCommands.h"
#include "ContentBrowserDelegates.h"
#include "ContentBrowserMenuContexts.h"
#include "ContentBrowserModule.h"
#include "Delegates/IDelegateInstance.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "IAdvancedRenamerModule.h"
#include "Providers/AdvancedRenamerAssetProvider.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"

#define LOCTEXT_NAMESPACE "AdvancedRenamerContentBrowserIntegration"

namespace UE::AdvancedRenamer::Private
{
	FDelegateHandle ContentBrowserDelegateHandle;

	TArray<FAssetData> GetContentBrowserSelectedAssets(FOnContentBrowserGetSelection InGetSelectionDelegate)
	{
		if (InGetSelectionDelegate.IsBound())
		{
			TArray<FAssetData> SelectedAssets;
			TArray<FString> SelectedPaths;
			InGetSelectionDelegate.Execute(SelectedAssets, SelectedPaths);
			return SelectedAssets;
		}
		return TArray<FAssetData>();
	}

	void OpenAdvancedRenamer(FOnContentBrowserGetSelection InGetSelectionDelegate)
	{
		const TArray<FAssetData> SelectedAssets = GetContentBrowserSelectedAssets(InGetSelectionDelegate);
		if (SelectedAssets.IsEmpty())
		{
			return;
		}

		const TSharedRef<FAdvancedRenamerAssetProvider> AssetProvider = MakeShared<FAdvancedRenamerAssetProvider>();
		AssetProvider->SetAssetList(SelectedAssets);

		TSharedPtr<SWidget> HostWidget = nullptr;

		IAdvancedRenamerModule::Get().OpenAdvancedRenamer(StaticCastSharedRef<IAdvancedRenamerProvider>(AssetProvider), HostWidget);
	}

	void AddMenuEntry(FToolMenuSection& InMenuSection)
	{
		FToolMenuEntry& BatchRenameMenuEntry = InMenuSection.AddMenuEntry(
			FAdvancedRenamerCommands::Get().BatchRenameObject,
			LOCTEXT("BatchRename", "Batch Rename"),
			LOCTEXT("AdvancedRenameTooltip", "Opens the Batch Renamer Panel to rename all selected assets."));

		BatchRenameMenuEntry.InsertPosition = FToolMenuInsert(TEXT("Rename"), EToolMenuInsertType::After);
	}

	void RegisterAssetMenu()
	{
		if (UToolMenus* ToolMenus = UToolMenus::Get())
		{
			FToolMenuOwnerScoped OwnerScoped(TEXT("AdvancedRenamer"));

			UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetContextMenu")->AddDynamicSection("BatchRenameDynamic", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
			{
				const UContentBrowserAssetContextMenuContext* Context = InMenu->FindContext<UContentBrowserAssetContextMenuContext>();
				if (Context && Context->bCanBeModified)
				{
					FToolMenuSection& MenuSection = InMenu->FindOrAddSection(TEXT("CommonAssetActions"));
					AddMenuEntry(MenuSection);
				}
			}));
		}
	}

	void OnExtendContentBrowserCommands(TSharedRef<FUICommandList> OutCommandList, FOnContentBrowserGetSelection InGetSelectionDelegate)
	{
		using namespace UE::AdvancedRenamer::Private;

		OutCommandList->MapAction(
			FAdvancedRenamerCommands::Get().BatchRenameObject,
			FUIAction(FExecuteAction::CreateStatic(&OpenAdvancedRenamer, InGetSelectionDelegate)));

		FInputBindingManager::Get().RegisterCommandList(FAdvancedRenamerCommands::Get().GetContextName(), OutCommandList);
	}

	void ExtendContentBrowserCommands()
	{
		if (FModuleManager::Get().IsModuleLoaded(TEXT("ContentBrowser")))
		{
			FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
			TArray<FContentBrowserCommandExtender>& CBCommandExtenderDelegates = ContentBrowserModule.GetAllContentBrowserCommandExtenders();
			CBCommandExtenderDelegates.Add(FContentBrowserCommandExtender::CreateStatic(&OnExtendContentBrowserCommands));
			ContentBrowserDelegateHandle = CBCommandExtenderDelegates.Last().GetHandle();
		}
	}
}

void FAdvancedRenamerContentBrowserIntegration::Initialize()
{
	using namespace UE::AdvancedRenamer::Private;

	// Extend the Content Browser Commands
	ExtendContentBrowserCommands();

	// Register the menu entry
	RegisterAssetMenu();
}

void FAdvancedRenamerContentBrowserIntegration::Shutdown()
{
	using namespace UE::AdvancedRenamer::Private;

	UToolMenus::UnregisterOwner(TEXT("AdvancedRenamer"));

	if (FModuleManager::Get().IsModuleLoaded(TEXT("ContentBrowser")))
	{
		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
		
		TArray<FContentBrowserCommandExtender>& CBCommandExtenderDelegates = ContentBrowserModule.GetAllContentBrowserCommandExtenders();
		CBCommandExtenderDelegates.RemoveAll([](const FContentBrowserCommandExtender& Delegate) { return Delegate.GetHandle() == ContentBrowserDelegateHandle; });
	}
}

#undef LOCTEXT_NAMESPACE
