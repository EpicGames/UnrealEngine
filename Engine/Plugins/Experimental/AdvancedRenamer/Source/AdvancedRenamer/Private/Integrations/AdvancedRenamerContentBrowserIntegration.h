// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FAdvancedRenamerContentBrowserIntegration
{
public:
	static void Initialize();
	static void Shutdown();
};
