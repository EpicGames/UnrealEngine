// Copyright Epic Games, Inc. All Rights Reserved.

#include "Integrations/AdvancedRenamerLevelEditorIntegration.h"
#include "AdvancedRenamerCommands.h"
#include "Containers/Array.h"
#include "Delegates/IDelegateInstance.h"
#include "EditorModeManager.h"
#include "GameFramework/Actor.h"
#include "IAdvancedRenamerModule.h"
#include "ILevelEditor.h"
#include "LevelEditor.h"
#include "Selection.h"
#include "Templates/SharedPointer.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "AdvancedRenamerLevelEditorIntegration"

namespace UE::AdvancedRenamer::Private
{
	FDelegateHandle LevelEditorCreatedDelegateHandle;

	TArray<AActor*> GetSelectedActors(TSharedRef<ILevelEditor> InLevelEditor)
	{
		USelection* ActorSelection = InLevelEditor->GetEditorModeManager().GetSelectedActors();

		if (!ActorSelection)
		{
			return {};
		}

		TArray<AActor*> SelectedActors;
		int32 Count = ActorSelection->GetSelectedObjects<AActor>(SelectedActors);

		return SelectedActors;
	}

	bool CanOpenAdvancedRenamer(TWeakPtr<ILevelEditor> InLevelEditorWeak)
	{
		TSharedPtr<ILevelEditor> LevelEditor = InLevelEditorWeak.Pin();

		if (!LevelEditor)
		{
			return false;
		}

		TArray<AActor*> SelectedActors = GetSelectedActors(LevelEditor.ToSharedRef());

		for (AActor* SelectedActor : SelectedActors)
		{
			if (IsValid(SelectedActor))
			{
				return true;
			}
		}

		return false;
	}

	void RenameSelectedActors(TWeakPtr<ILevelEditor> InLevelEditorWeak)
	{
		TSharedPtr<ILevelEditor> LevelEditor = InLevelEditorWeak.Pin();

		if (!LevelEditor)
		{
			return;
		}

		TArray<AActor*> SelectedActors = GetSelectedActors(LevelEditor.ToSharedRef());

		IAdvancedRenamerModule::Get().OpenAdvancedRenamerForActors(SelectedActors, StaticCastSharedPtr<IToolkitHost>(LevelEditor));
	}

	void RenameSharedClassActors(TWeakPtr<ILevelEditor> InLevelEditorWeak)
	{
		TSharedPtr<ILevelEditor> LevelEditor = InLevelEditorWeak.Pin();

		if (!LevelEditor)
		{
			return;
		}

		IAdvancedRenamerModule& AdvancedRenamerModule = IAdvancedRenamerModule::Get();

		TArray<AActor*> SelectedActors = GetSelectedActors(LevelEditor.ToSharedRef());
		SelectedActors = AdvancedRenamerModule.GetActorsSharingClassesInWorld(SelectedActors);

		AdvancedRenamerModule.OpenAdvancedRenamerForActors(SelectedActors, StaticCastSharedPtr<IToolkitHost>(LevelEditor));
	}

	void OnLevelEditorCreated(TSharedPtr<ILevelEditor> InLevelEditor)
	{
		const FAdvancedRenamerCommands& AdvRenCommands = FAdvancedRenamerCommands::Get();

		const TSharedPtr<FUICommandList>& LevelEditorAction = InLevelEditor->GetLevelEditorActions();
		if (LevelEditorAction.IsValid())
		{
			LevelEditorAction->MapAction(
				AdvRenCommands.BatchRenameObject,
				FExecuteAction::CreateStatic(&RenameSelectedActors, InLevelEditor.ToWeakPtr()),
				FCanExecuteAction::CreateStatic(&CanOpenAdvancedRenamer, InLevelEditor.ToWeakPtr())
			);

			LevelEditorAction->MapAction(
				AdvRenCommands.BatchRenameSharedClassActors,
				FExecuteAction::CreateStatic(&RenameSharedClassActors, InLevelEditor.ToWeakPtr()),
				FCanExecuteAction::CreateStatic(&CanOpenAdvancedRenamer, InLevelEditor.ToWeakPtr())
			);
		}

		FInputBindingManager::Get().RegisterCommandList(AdvRenCommands.GetContextName(), LevelEditorAction.ToSharedRef());
	}
}

static const TArray<FName, TInlineAllocator<2>> Menus = {
	"LevelEditor.ActorContextMenu.EditSubMenu",
	"LevelEditor.LevelEditorSceneOutliner.ContextMenu.EditSubMenu"
};

void FAdvancedRenamerLevelEditorIntegration::Initialize()
{
	using namespace UE::AdvancedRenamer::Private;

	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	LevelEditorCreatedDelegateHandle = LevelEditorModule.OnLevelEditorCreated().AddStatic(&OnLevelEditorCreated);
	InitializeMenu();
}

void FAdvancedRenamerLevelEditorIntegration::Shutdown()
{
	using namespace UE::AdvancedRenamer::Private;

	if (FLevelEditorModule* LevelEditorModule = FModuleManager::LoadModulePtr<FLevelEditorModule>("LevelEditor"))
	{
		LevelEditorModule->OnLevelEditorCreated().Remove(LevelEditorCreatedDelegateHandle);
		LevelEditorCreatedDelegateHandle.Reset();
	}
	ShutdownMenu();
}

void FAdvancedRenamerLevelEditorIntegration::InitializeMenu()
{
	const FAdvancedRenamerCommands& AdvRenCommands = FAdvancedRenamerCommands::Get();

	const TAttribute<FText> TextAttribute;

	for (const FName& Menu : Menus)
	{
		UToolMenu* ToolMenu = UToolMenus::Get()->ExtendMenu(Menu);
		FToolMenuSection& Section = ToolMenu->FindOrAddSection(NAME_None);

		Section.AddMenuEntry(
			AdvRenCommands.BatchRenameObject,
			LOCTEXT("BatchRename", "Rename Selected Actors"),
			LOCTEXT("BatchRenameToolTip", "Opens the Batch Renamer Panel to rename all selected actors.")
			
		);

		Section.AddMenuEntry(
			AdvRenCommands.BatchRenameSharedClassActors,
			LOCTEXT("BatchRenameByClass", "Rename Actors of Selected Actor Classes")
		);
	}
}

void FAdvancedRenamerLevelEditorIntegration::ShutdownMenu()
{
	if (UToolMenus* ToolMenus = UToolMenus::Get())
	{
		for (const FName& Menu : Menus)
		{
			ToolMenus->RemoveEntry(Menu, NAME_None, TEXT("RenameSelectedActors"));
			ToolMenus->RemoveEntry(Menu, NAME_None, TEXT("RenameSharedClassActors"));
		}
	}
}

#undef LOCTEXT_NAMESPACE
