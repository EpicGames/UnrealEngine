// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FAdvancedRenamerLevelEditorIntegration
{
public:
	static void Initialize();
	static void Shutdown();

private:
	static void InitializeMenu();
	static void ShutdownMenu();
};
