// Copyright Epic Games, Inc. All Rights Reserved.

#include "Providers/AdvancedRenamerActorProvider.h"
#include "GameFramework/Actor.h"

FAdvancedRenamerActorProvider::FAdvancedRenamerActorProvider()
{
}

FAdvancedRenamerActorProvider::~FAdvancedRenamerActorProvider()
{
}

void FAdvancedRenamerActorProvider::SetActorList(const TArray<TWeakObjectPtr<AActor>>& InActorList)
{
	ActorList.Empty();
	ActorList.Append(InActorList);
}

void FAdvancedRenamerActorProvider::AddActorList(const TArray<TWeakObjectPtr<AActor>>& InActorList)
{
	ActorList.Append(InActorList);
}

void FAdvancedRenamerActorProvider::AddActorData(AActor* InActor)
{
	ActorList.Add(InActor);
}

AActor* FAdvancedRenamerActorProvider::GetActor(int32 InIndex) const
{
	if (!ActorList.IsValidIndex(InIndex))
	{
		return nullptr;
	}

	return ActorList[InIndex].Get();
}

int32 FAdvancedRenamerActorProvider::Num() const
{
	return ActorList.Num();
}

bool FAdvancedRenamerActorProvider::IsValidIndex(int32 InIndex) const
{
	AActor* Actor = GetActor(InIndex);

	return IsValid(Actor);
}

FString FAdvancedRenamerActorProvider::GetOriginalName(int32 InIndex) const
{
	AActor* Actor = GetActor(InIndex);

	if (!IsValid(Actor))
	{
		return "";
	}

	return Actor->GetActorNameOrLabel();
}

uint32 FAdvancedRenamerActorProvider::GetHash(int32 InIndex) const
{
	AActor* Actor = GetActor(InIndex);

	if (!IsValid(Actor))
	{
		return 0;
	}

	return GetTypeHash(Actor);
}

bool FAdvancedRenamerActorProvider::RemoveIndex(int32 InIndex)
{
	if (!ActorList.IsValidIndex(InIndex))
	{
		return false;
	}

	ActorList.RemoveAt(InIndex);
	return true;
}

bool FAdvancedRenamerActorProvider::CanRename(int32 InIndex) const
{
	AActor* Actor = GetActor(InIndex);

	if (!IsValid(Actor))
	{
		return false;
	}

	return true;
}

bool FAdvancedRenamerActorProvider::BeginRename()
{
	ActorToNewNameList.Reserve(Num());
	return true;
}

bool FAdvancedRenamerActorProvider::PrepareRename(int32 InIndex, const FString& InNewName)
{
	AActor* Actor = GetActor(InIndex);

	if (!IsValid(Actor))
	{
		return false;
	}

	ActorToNewNameList.Add({ Actor , InNewName });
	return true;
}

bool FAdvancedRenamerActorProvider::ExecuteRename()
{
	for (const TTuple<AActor*, FString>& ActorToNewName : ActorToNewNameList)
	{
		ActorToNewName.Key->SetActorLabel(ActorToNewName.Value, /* bMarkDirty */ true);
	}
	return true;
}

bool FAdvancedRenamerActorProvider::EndRename()
{
	ActorToNewNameList.Empty();
	return true;
}
