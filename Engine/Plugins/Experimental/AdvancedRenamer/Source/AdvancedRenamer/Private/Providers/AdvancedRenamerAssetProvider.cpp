// Copyright Epic Games, Inc. All Rights Reserved.

#include "Providers/AdvancedRenamerAssetProvider.h"
#include "AssetToolsModule.h"
#include "IAssetTools.h"

#define LOCTEXT_NAMESPACE "AdvancedRenamerAssetProvider"

FAdvancedRenamerAssetProvider::FAdvancedRenamerAssetProvider()
{
}

FAdvancedRenamerAssetProvider::~FAdvancedRenamerAssetProvider()
{
}

void FAdvancedRenamerAssetProvider::SetAssetList(const TArray<FAssetData>& InAssetList)
{
	AssetList.Empty();
	AssetList.Append(InAssetList);
}

void FAdvancedRenamerAssetProvider::AddAssetList(const TArray<FAssetData>& InAssetList)
{
	AssetList.Append(InAssetList);
}

void FAdvancedRenamerAssetProvider::AddAssetData(const FAssetData& InAsset)
{
	AssetList.Add(InAsset);
}

UObject* FAdvancedRenamerAssetProvider::GetAsset(int32 InIndex) const
{
	if (!AssetList.IsValidIndex(InIndex))
	{
		return nullptr;
	}

	return AssetList[InIndex].GetAsset();
}

int32 FAdvancedRenamerAssetProvider::Num() const
{
	return AssetList.Num();
}

bool FAdvancedRenamerAssetProvider::IsValidIndex(int32 InIndex) const
{
	UObject* Asset = GetAsset(InIndex);

	return IsValid(Asset) && Asset->IsAsset();
}

FString FAdvancedRenamerAssetProvider::GetOriginalName(int32 InIndex) const
{
	UObject* Asset = GetAsset(InIndex);

	if (!IsValid(Asset))
	{
		return "";
	}

	return Asset->GetName();
}

uint32 FAdvancedRenamerAssetProvider::GetHash(int32 InIndex) const
{
	UObject* Asset = GetAsset(InIndex);

	if (!IsValid(Asset))
	{
		return 0;
	}

	return GetTypeHash(Asset);
}

bool FAdvancedRenamerAssetProvider::RemoveIndex(int32 InIndex)
{
	if (!AssetList.IsValidIndex(InIndex))
	{
		return false;
	}

	AssetList.RemoveAt(InIndex);
	return true;
}

bool FAdvancedRenamerAssetProvider::CanRename(int32 InIndex) const
{
	UObject* Asset = GetAsset(InIndex);

	if (!IsValid(Asset))
	{
		return false;
	}

	return true;
}

bool FAdvancedRenamerAssetProvider::BeginRename()
{
	AssetRenameDataList.Reserve(Num());
	return true;
}

bool FAdvancedRenamerAssetProvider::PrepareRename(int32 InIndex, const FString& InNewName)
{
	UObject* Asset = GetAsset(InIndex);

	if (!IsValid(Asset))
	{
		return false;
	}

	FString PackagePath = Asset->GetPathName();
	PackagePath = FPaths::GetPath(PackagePath);

	constexpr bool bOnlyFixSoftReferences = false;
	constexpr bool bAlsoRenameLocalizedVariants = true;
	AssetRenameDataList.Add(FAssetRenameData(Asset, PackagePath, InNewName, bOnlyFixSoftReferences, bAlsoRenameLocalizedVariants));

	return true;
}

bool FAdvancedRenamerAssetProvider::ExecuteRename()
{
	IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();

	constexpr bool bAutoCheckout = false;
	return AssetTools.RenameAssetsWithDialog(AssetRenameDataList, bAutoCheckout) != EAssetRenameResult::Failure;
}

bool FAdvancedRenamerAssetProvider::EndRename()
{
	AssetRenameDataList.Empty();
	return true;
}

#undef LOCTEXT_NAMESPACE
