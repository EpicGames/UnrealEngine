// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Providers/IAdvancedRenamerProvider.h"
#include "AssetRegistry/AssetData.h"
#include "Containers/Array.h"

struct FAssetData;
struct FAssetRenameData;

class FAdvancedRenamerAssetProvider : public IAdvancedRenamerProvider
{
public:
	FAdvancedRenamerAssetProvider();
	virtual ~FAdvancedRenamerAssetProvider() override;

	void SetAssetList(const TArray<FAssetData>& InAssetList);
	void AddAssetList(const TArray<FAssetData>& InAssetList);
	void AddAssetData(const FAssetData& InAsset);
	UObject* GetAsset(int32 InIndex) const;

protected:
	//~ Begin IAdvancedRenamerProvider
	virtual int32 Num() const override;
	virtual bool IsValidIndex(int32 InIndex) const override;
	virtual uint32 GetHash(int32 InIndex) const override;;
	virtual FString GetOriginalName(int32 InIndex) const override;
	virtual bool RemoveIndex(int32 InIndex) override;
	virtual bool CanRename(int32 InIndex) const override;

	virtual bool BeginRename() override;
	virtual bool PrepareRename(int32 InIndex, const FString& InNewName) override;
	virtual bool ExecuteRename() override;
	virtual bool EndRename() override;
	//~ End IAdvancedRenamerProvider

	TArray<FAssetData> AssetList;
	TArray<FAssetRenameData> AssetRenameDataList;
};
