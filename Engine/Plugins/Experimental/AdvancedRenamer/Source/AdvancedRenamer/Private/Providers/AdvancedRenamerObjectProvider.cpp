// Copyright Epic Games, Inc. All Rights Reserved.

#include "Providers/AdvancedRenamerObjectProvider.h"
#include "UObject/Object.h"

FAdvancedRenamerObjectProvider::FAdvancedRenamerObjectProvider()
{
}

FAdvancedRenamerObjectProvider::~FAdvancedRenamerObjectProvider()
{
}

void FAdvancedRenamerObjectProvider::SetObjectList(const TArray<TWeakObjectPtr<UObject>>& InObjectList)
{
	ObjectList.Empty();
	ObjectList.Append(InObjectList);
}

void FAdvancedRenamerObjectProvider::AddObjectList(const TArray<TWeakObjectPtr<UObject>>& InObjectList)
{
	ObjectList.Append(InObjectList);
}

void FAdvancedRenamerObjectProvider::AddObjectData(UObject* InObject)
{
	ObjectList.Add(InObject);
}

UObject* FAdvancedRenamerObjectProvider::GetObject(int32 InIndex) const
{
	if (!ObjectList.IsValidIndex(InIndex))
	{
		return nullptr;
	}

	return ObjectList[InIndex].Get();
}

int32 FAdvancedRenamerObjectProvider::Num() const
{
	return ObjectList.Num();
}

bool FAdvancedRenamerObjectProvider::IsValidIndex(int32 InIndex) const
{
	UObject* Object = GetObject(InIndex);

	return IsValid(Object);
}

FString FAdvancedRenamerObjectProvider::GetOriginalName(int32 InIndex) const
{
	UObject* Object = GetObject(InIndex);

	if (!IsValid(Object))
	{
		return "";
	}

	return Object->GetName();
}

uint32 FAdvancedRenamerObjectProvider::GetHash(int32 InIndex) const
{
	UObject* Object = GetObject(InIndex);

	if (!IsValid(Object))
	{
		return 0;
	}

	return GetTypeHash(Object);
}

bool FAdvancedRenamerObjectProvider::RemoveIndex(int32 InIndex)
{
	if (!ObjectList.IsValidIndex(InIndex))
	{
		return false;
	}

	ObjectList.RemoveAt(InIndex);
	return true;
}

bool FAdvancedRenamerObjectProvider::CanRename(int32 InIndex) const
{
	UObject* Object = GetObject(InIndex);

	if (!IsValid(Object))
	{
		return false;
	}

	return true;
}

bool FAdvancedRenamerObjectProvider::BeginRename()
{
	ObjectToNewNameList.Reserve(Num());
	return true;
}

bool FAdvancedRenamerObjectProvider::PrepareRename(int32 InIndex, const FString& InNewName)
{
	UObject* Object = GetObject(InIndex);

	if (!IsValid(Object))
	{
		return false;
	}

	ObjectToNewNameList.Add({ Object, InNewName });

	return true;
}

bool FAdvancedRenamerObjectProvider::ExecuteRename()
{
	bool bAllSuccess = true;
	for (const TTuple<UObject*, FString>& ObjectToNewName : ObjectToNewNameList)
	{
		// We have to run the REN_Test here to prevent the other renamed objects from making this rename fail
		if (ObjectToNewName.Key->Rename(*ObjectToNewName.Value, nullptr, REN_Test))
		{
			bAllSuccess &= ObjectToNewName.Key->Rename(*ObjectToNewName.Value);
		}
		else
		{
			bAllSuccess = false;
		}
	}
	return bAllSuccess;
}

bool FAdvancedRenamerObjectProvider::EndRename()
{
	ObjectToNewNameList.Empty();
	return true;
}
