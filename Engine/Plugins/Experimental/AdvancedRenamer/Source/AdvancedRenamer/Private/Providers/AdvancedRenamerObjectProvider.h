// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Providers/IAdvancedRenamerProvider.h"
#include "Containers/Array.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FString;
class UObject;

class FAdvancedRenamerObjectProvider : public IAdvancedRenamerProvider
{
public:
	FAdvancedRenamerObjectProvider();
	virtual ~FAdvancedRenamerObjectProvider() override;

	void SetObjectList(const TArray<TWeakObjectPtr<UObject>>& InObjectList);
	void AddObjectList(const TArray<TWeakObjectPtr<UObject>>& InObjectList);
	void AddObjectData(UObject* InObject);
	UObject* GetObject(int32 InIndex) const;

protected:
	//~ Begin IAdvancedRenamerProvider
	virtual int32 Num() const override;
	virtual bool IsValidIndex(int32 InIndex) const override;
	virtual uint32 GetHash(int32 InIndex) const override;;
	virtual FString GetOriginalName(int32 InIndex) const override;
	virtual bool RemoveIndex(int32 InIndex) override;
	virtual bool CanRename(int32 InIndex) const override;

	virtual bool BeginRename() override;
	virtual bool PrepareRename(int32 InIndex, const FString& InNewName) override;
	virtual bool ExecuteRename() override;
	virtual bool EndRename() override;
	//~ End IAdvancedRenamerProvider

	TArray<TWeakObjectPtr<UObject>> ObjectList;
	TArray<TTuple<UObject*, FString>> ObjectToNewNameList;
};
