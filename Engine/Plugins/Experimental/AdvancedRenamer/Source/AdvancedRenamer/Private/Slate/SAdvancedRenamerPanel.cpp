// Copyright Epic Games, Inc. All Rights Reserved.

#include "Slate/SAdvancedRenamerPanel.h"

#include "AdvancedRenamerModule.h"
#include "AdvancedRenamerSections/IAdvancedRenamerSection.h"
#include "AdvancedRenamerStyle.h"
#include "EngineAnalytics.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GenericPlatform/GenericPlatformMath.h"
#include "IAdvancedRenamer.h"
#include "ScopedTransaction.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Utils/AdvancedRenamerSlateUtils.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SAdvancedRenamerPanel"

class SAdvancedRenamerPreviewListRow : public SMultiColumnTableRow<TSharedPtr<FAdvancedRenamerPreview>>
{
public:
	SLATE_BEGIN_ARGS(SAdvancedRenamerPreviewListRow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<SAdvancedRenamerPanel> InRenamePanel,
		const TSharedRef<STableViewBase>& InOwnerTableView, TSharedPtr<FAdvancedRenamerPreview> InRowItem)
	{
		PanelWeak = InRenamePanel;
		ItemWeak = InRowItem;

		SMultiColumnTableRow<TSharedPtr<FAdvancedRenamerPreview>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
		SetBorderImage(FAdvancedRenamerStyle::Get().GetBrush("AdvancedRenamer.Style.BackgroundBorder"));
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
	{
		using namespace AdvancedRenamerSlateUtils::Default;

		if (InColumnName != OriginalNameColumnName
			&& InColumnName != NewNameColumnName)
		{
			return SNullWidget::NullWidget;
		}

		TSharedPtr<SAdvancedRenamerPanel> Panel = PanelWeak.Pin();

		if (!Panel.IsValid())
		{
			return SNullWidget::NullWidget;
		}

		TSharedPtr<FAdvancedRenamerPreview> Item = ItemWeak.Pin();

		if (!Item.IsValid())
		{
			return SNullWidget::NullWidget;
		}

		TSharedRef<STextBlock> TextBlock = SNew(STextBlock)
			.Font(FAdvancedRenamerStyle::Get().GetFontStyle("AdvancedRenamer.Style.RegularFont"));

		TSharedRef<SBox> TextBlockBox = SNew(SBox)
			.Padding(FMargin(8.f, 6.f, 0.f, 6.f));

		if (InColumnName == OriginalNameColumnName)
		{
			TextBlock->SetText(FText::FromString(Item->OriginalName));
		}
		else if (InColumnName == NewNameColumnName)
		{
			TextBlock->SetText(FText::FromString(Item->NewName));
		}

		TextBlockBox->SetContent(TextBlock);
		return TextBlockBox;
	}

protected:
	TWeakPtr<SAdvancedRenamerPanel> PanelWeak;
	TWeakPtr<FAdvancedRenamerPreview> ItemWeak;
};

void SAdvancedRenamerPanel::Construct(const FArguments& InArgs, const TSharedRef<IAdvancedRenamer>& InRenamer)
{
	Renamer = InRenamer;

	SortMode = EColumnSortMode::None;
	PreviewList = InRenamer->GetSortablePreviews();

	CommandList = MakeShared<FUICommandList>();
	CommandList->MapAction(
		FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &SAdvancedRenamerPanel::RemoveSelectedObjects),
		FCanExecuteAction()
	);

	using namespace AdvancedRenamerSlateUtils::Default;

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			[
				SNew(SBox)
				[
					SNew(SSplitter)
					.PhysicalSplitterHandleSize(2.f)
					.Style(&FAdvancedRenamerStyle::Get().GetWidgetStyle<FSplitterStyle>("AdvancedRenamer.Style.Splitter"))

					+ SSplitter::Slot()
					.Value(0.505f)
					.MinSize(365.f)
					[
						SAssignNew(LeftSideVerticalBox, SVerticalBox)
					]

					+ SSplitter::Slot()
					.Value(0.495f)
					[
						SAssignNew(RightSideBox, SBox)
					]
				]
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAdvancedRenamerStyle::Get().GetBrush("AdvancedRenamer.Style.BackgroundBorder"))
			[
				SAssignNew(ApplyResetCancelBox, SBox)
			]
		]
	];

	TArray<TSharedPtr<IAdvancedRenamerSection>> RegisteredSections = IAdvancedRenamerModule::Get().GetRegisteredSections();
	for (int32 Index = 0; Index < RegisteredSections.Num(); Index++)
	{
		RegisteredSections[Index]->Init(InRenamer);
		FMargin Padding = FMargin(0.f, 1.f);

		if (Index == 0)
		{
			Padding = FMargin(0.f, 0.f, 0.f, 1.f);
		}
		else if (Index == RegisteredSections.Num() - 1)
		{
			Padding = FMargin(0.f, 1.f, 0.f, 0.f);
		}

		LeftSideVerticalBox->AddSlot()
			.Padding(Padding)
			.AutoHeight()
			[
				RegisteredSections[Index]->GetWidget()
			];
	}

	CreateRightPanel();
	CreateApplyResetCancelPanel();

	Renamer->MarkDirty();

	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.AdvancedRenamer.Opened"));
	}
}

void SAdvancedRenamerPanel::CreateApplyResetCancelPanel()
{
	using namespace AdvancedRenamerSlateUtils::Default;

	ApplyResetCancelBox->SetContent(SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			[
				SNew(SSpacer)
			]

			+ SHorizontalBox::Slot()
			.Padding(ApplyButtonPadding)
			.HAlign(HAlign_Right)
			.AutoWidth()
			[
				CreateApplyButton()
			]

			+ SHorizontalBox::Slot()
			.Padding(ResetButtonPadding)
			.HAlign(HAlign_Right)
			.AutoWidth()
			[
				CreateResetButton()
			]

			+ SHorizontalBox::Slot()
			.Padding(CancelButtonPadding)
			.HAlign(HAlign_Right)
			.AutoWidth()
			[
				CreateCancelButton()
			]);
}

TSharedRef<SWidget> SAdvancedRenamerPanel::CreateApplyButton()
{
	return SNew(SBox)
		.HeightOverride(25)
		.WidthOverride(75)
		[
			SNew(SButton)
			.IsEnabled(this, &SAdvancedRenamerPanel::IsApplyButtonEnabled)
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("PrimaryButton"))
			.ContentPadding(FMargin(-2.f, -1.f))
			.OnClicked(this, &SAdvancedRenamerPanel::OnApplyButtonClicked)
			.Content()
			[
				SNew(SBox)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Justification(ETextJustify::Center)
					.Text(LOCTEXT("AR_Apply", "Apply"))
				]
			]
		];
}

TSharedRef<SWidget> SAdvancedRenamerPanel::CreateResetButton()
{
	return SNew(SBox)
		.HeightOverride(25)
		.WidthOverride(75)
		[
			SNew(SButton)
			.ContentPadding(FMargin(-2.f, -1.f))
			.OnClicked(this, &SAdvancedRenamerPanel::OnResetButtonClicked)
			.Content()
			[
				SNew(SBox)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Justification(ETextJustify::Center)
					.Text(LOCTEXT("AR_Reset", "Reset"))
				]
			]
		];
}

TSharedRef<SWidget> SAdvancedRenamerPanel::CreateCancelButton()
{
	return SNew(SBox)
		.HeightOverride(25)
		.WidthOverride(75)
		[
			SNew(SButton)
			.ContentPadding(FMargin(-2.f, -1.f))
			.OnClicked(this, &SAdvancedRenamerPanel::OnCancelButtonClicked)
			.Content()
			[
				SNew(SBox)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Justification(ETextJustify::Center)
					.Text(LOCTEXT("AR_Cancel", "Cancel"))
				]
			]
		];
}

void SAdvancedRenamerPanel::CreateRightPanel()
{
	using namespace AdvancedRenamerSlateUtils::Default;

	RightSideBox->SetContent(SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		[
			CreateRenamePreview()
		]);
}

TSharedRef<SWidget> SAdvancedRenamerPanel::CreateRenamePreview()
{
	using namespace AdvancedRenamerSlateUtils::Default;

	return SNew(SBorder)
		.BorderImage(FAdvancedRenamerStyle::Get().GetBrush("AdvancedRenamer.Style.BackgroundBorder"))
		.Content()
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.FillHeight(1.f)
			[
				SAssignNew(RenamePreviewListBox, SBox)
				.Content()
				[
					SAssignNew(RenamePreviewList, SListView<TSharedPtr<FAdvancedRenamerPreview>>)
					.ListViewStyle(&FAdvancedRenamerStyle::Get().GetWidgetStyle<FTableViewStyle>("AdvancedRenamer.Style.ListView"))
					.ListItemsSource(&PreviewList)
					.OnGenerateRow(this, &SAdvancedRenamerPanel::OnGenerateRowForList)
					.HeaderRow(

						SAssignNew(RenamePreviewListHeaderRow, SHeaderRow)
						.Style(&FAdvancedRenamerStyle::Get().GetWidgetStyle<FHeaderRowStyle>("AdvancedRenamer.Style.HeaderRow"))

						+ SHeaderRow::Column(OriginalNameColumnName)
						.HeaderContentPadding(FMargin(8.f, 2.f))
						.DefaultLabel(LOCTEXT("AR_Old", "Old"))
						.SortMode(this, &SAdvancedRenamerPanel::GetColumnSortMode)
						.OnSort(this, &SAdvancedRenamerPanel::OnColumnSortModeChanged)
						.FillWidth(0.5f)

						+ SHeaderRow::Column(NewNameColumnName)
						.HeaderContentPadding(FMargin(8.f, 2.f))
						.DefaultLabel(LOCTEXT("AR_New", "New"))
						.FillWidth(0.5f)
					)
					.OnKeyDownHandler(this, &SAdvancedRenamerPanel::OnListViewKeyDown)
					.OnContextMenuOpening(this, &SAdvancedRenamerPanel::GenerateListViewContextMenu)
				]
			]
		];
}

bool SAdvancedRenamerPanel::CloseWindow()
{
	TSharedPtr<SWindow> CurrentWindow = FSlateApplication::Get().FindWidgetWindow(SharedThis(this));

	if (CurrentWindow.IsValid())
	{
		CurrentWindow->RequestDestroyWindow();
		return true;
	}

	return false;
}

void SAdvancedRenamerPanel::RefreshListViewAndUpdate(const double InCurrentTime)
{
	const int32 CurrentCount = Renamer->Num();

	Renamer->UpdatePreviews();

	if (CurrentCount != Renamer->Num())
	{
		RenamePreviewList->RequestListRefresh();
	}

	RenamePreviewList->RebuildList();

	ListLastUpdateTime = InCurrentTime;
}

void SAdvancedRenamerPanel::RemoveSelectedObjects()
{
	if (!RenamePreviewList.IsValid() || RenamePreviewList->GetNumItemsSelected() == 0)
	{
		return;
	}

	TArray<TSharedPtr<FAdvancedRenamerPreview>> SelectedItems = RenamePreviewList->GetSelectedItems();

	if (SelectedItems.Num() == 0)
	{
		return;
	}

	bool bMadeChange = false;

	for (int32 SelectedIndex = 0; SelectedIndex < SelectedItems.Num(); ++SelectedIndex)
	{
		if (!SelectedItems[SelectedIndex].IsValid())
		{
			continue;
		}

		const int32 PreviewIndex = Renamer->FindHash(SelectedItems[SelectedIndex]->Hash);

		if (PreviewIndex == INDEX_NONE)
		{
			continue;
		}

		if (Renamer->RemoveIndex(PreviewIndex))
		{
			bMadeChange = true;
		}
	}

	SelectedItems.Empty();

	if (bMadeChange)
	{
		if (Renamer->Num() == 0)
		{
			if (CloseWindow())
			{
				return;
			}
		}

		RenamePreviewList->RequestListRefresh();
	}
}

void SAdvancedRenamerPanel::Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(InAllottedGeometry, InCurrentTime, InDeltaTime);

	if (ListLastUpdateTime == 0)
	{
		ListLastUpdateTime = InCurrentTime;
	}
	else if (Renamer->IsDirty() && InCurrentTime >= (ListLastUpdateTime + MinUpdateFrequency))
	{
		RefreshListViewAndUpdate(InCurrentTime);
	}
}

EColumnSortMode::Type SAdvancedRenamerPanel::GetColumnSortMode() const
{
	return SortMode;
}

void SAdvancedRenamerPanel::OnColumnSortModeChanged(EColumnSortPriority::Type InSortPriority, const FName& InName, EColumnSortMode::Type InSortMode)
{
	if (SortMode == EColumnSortMode::Descending && InSortMode == EColumnSortMode::Ascending)
	{
		SortMode = EColumnSortMode::None;
	}
	else
	{
		SortMode = InSortMode;
	}

	if (Renamer.IsValid())
	{
		TArray<TSharedPtr<FAdvancedRenamerPreview>>& SortablePreview = Renamer->GetSortablePreviews();
		if (SortMode != EColumnSortMode::None)
		{
			EColumnSortMode::Type TempSortMode = SortMode;
			auto ComparePreviewList = [TempSortMode](const TSharedPtr<FAdvancedRenamerPreview>& A, const TSharedPtr<FAdvancedRenamerPreview>& B)
			{
				if (!A.IsValid() || !B.IsValid())
				{
					return false;
				}

				const bool CompareResult = A->GetNameForSort().Compare(B->GetNameForSort()) <= 0;
				return TempSortMode == EColumnSortMode::Ascending ? CompareResult : !CompareResult;
			};

			SortablePreview.Sort(ComparePreviewList);
			PreviewList = SortablePreview;
		}
		else
		{
			Renamer->ResetSortablePreviews();
			PreviewList = Renamer->GetSortablePreviews();
		}

		Renamer->MarkDirty();
		RenamePreviewList->RequestListRefresh();
	}
}

TSharedRef<ITableRow> SAdvancedRenamerPanel::OnGenerateRowForList(TSharedPtr<FAdvancedRenamerPreview> InItem, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return SNew(SAdvancedRenamerPreviewListRow, SharedThis(this), InOwnerTable, InItem);
}

FReply SAdvancedRenamerPanel::OnListViewKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList.IsValid() && CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

TSharedPtr<SWidget> SAdvancedRenamerPanel::GenerateListViewContextMenu()
{
	if (!RenamePreviewList.IsValid())
	{
		return nullptr;
	}

	if (RenamePreviewList->GetNumItemsSelected() == 0)
	{
		return nullptr;
	}

	FMenuBuilder MenuBuilder(true, CommandList.ToSharedRef());

	MenuBuilder.BeginSection("Actions", LOCTEXT("Actions", "Actions"));
	{
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete, NAME_None, LOCTEXT("RemoveObject", "Remove Object"));
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

bool SAdvancedRenamerPanel::IsApplyButtonEnabled() const
{
	return !Renamer->IsDirty() && Renamer->HasRenames();
}

FReply SAdvancedRenamerPanel::OnApplyButtonClicked()
{
	FScopedTransaction Transaction(LOCTEXT("AdvancedRenamerRename", "Batch Renamer Rename"));

	if (!Renamer->Execute())
	{
		FNotificationInfo Info(LOCTEXT("AR_RenameIssueNotification", "Errors occured while applying the rename.\nSee Output Log for more information on why the rename failed."));
		Info.ExpireDuration = 5.0f;
		const TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
		if (Notification.IsValid())
		{
			Notification->SetCompletionState(SNotificationItem::CS_Fail);
		}
	}

	CloseWindow();
	return FReply::Handled();
}

FReply SAdvancedRenamerPanel::OnResetButtonClicked()
{
	TArray<TSharedPtr<IAdvancedRenamerSection>> RegisteredSections = IAdvancedRenamerModule::Get().GetRegisteredSections();
	for (const TSharedPtr<IAdvancedRenamerSection>& Section : RegisteredSections)
	{
		Section->ResetToDefault();
	}

	if (Renamer.IsValid())
	{
		Renamer->MarkDirty();
	}

	return FReply::Handled();
}

FReply SAdvancedRenamerPanel::OnCancelButtonClicked()
{
	CloseWindow();
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
