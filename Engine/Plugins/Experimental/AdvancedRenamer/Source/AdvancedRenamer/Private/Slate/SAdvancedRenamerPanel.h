// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Templates/SharedPointerFwd.h"

class FUICommandList;
class IAdvancedRenamer;
class ITableRow;
class SBox;
template <typename InItemType>
class SListView;
class STableViewBase;
class SVerticalBox;
class SWidget;
struct FAdvancedRenamerPreview;
struct FGeometry;
struct FKeyEvent;

class SAdvancedRenamerPanel : public SCompoundWidget
{
	friend class SAdvancedRenamerPreviewListRow;

public:
	SLATE_BEGIN_ARGS(SAdvancedRenamerPanel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<IAdvancedRenamer>& InRenamer);

private:
	/** Create Apply Reset and Cancel bottom panel */
	void CreateApplyResetCancelPanel();

	/** Create Apply button */
	TSharedRef<SWidget> CreateApplyButton();

	/** Create Reset button */
	TSharedRef<SWidget> CreateResetButton();

	/** Create Cancel button */
	TSharedRef<SWidget> CreateCancelButton();

	/** Create the right side panel */
	void CreateRightPanel();

	/** Create the rename preview */
	TSharedRef<SWidget> CreateRenamePreview();

	/** Close the Renamer window */
	bool CloseWindow();

	/** Refresh and Update the Preview */
	void RefreshListViewAndUpdate(const double InCurrentTime);

	/** Remove selected entries from the Renamer*/
	void RemoveSelectedObjects();

	//~ Begin SWidget Interface
	virtual void Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End SWidget Interface

	/** Get the current Column SortMode */
	EColumnSortMode::Type GetColumnSortMode() const;

	/** Called when the SortMode change */
	void OnColumnSortModeChanged(EColumnSortPriority::Type InSortPriority, const FName& InName, EColumnSortMode::Type InSortMode);
	
	/** Callback to generate ListBoxRows */
	TSharedRef<ITableRow> OnGenerateRowForList(TSharedPtr<FAdvancedRenamerPreview> InItem, const TSharedRef<STableViewBase>& InOwnerTable);

	/** Handle the click on the ListView */
	FReply OnListViewKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent);

	/** Generate the ListView menu */
	TSharedPtr<SWidget> GenerateListViewContextMenu();

	/** Whether or not the ApplyButton is enabled */
	bool IsApplyButtonEnabled() const;

	/** Called when clicking on the Apply button */
	FReply OnApplyButtonClicked();

	/** Called when clicking on the Reset button */
	FReply OnResetButtonClicked();

	/** Called when clicking on the Cancel button */
	FReply OnCancelButtonClicked();

private:
	/** Min update frequency, used in the tick to avoid updating the renamer too often */
	static constexpr double MinUpdateFrequency = 0.1f;

	/** Current SortMode for the List */
	EColumnSortMode::Type SortMode;

	/** Preview list used for the ListView */
	TArray<TSharedPtr<FAdvancedRenamerPreview>> PreviewList;

	/** Renamer instance */
	TSharedPtr<IAdvancedRenamer> Renamer;

	/** Command list of the Renamer Panel */
	TSharedPtr<FUICommandList> CommandList;

	/** Last update time of the list */
	double ListLastUpdateTime = 0;

	/** Sections VerticalBox */
	TSharedPtr<SVerticalBox> LeftSideVerticalBox;

	/** RightSide container Box */
	TSharedPtr<SBox> RightSideBox;

	/** Bottom panel button container Box */
	TSharedPtr<SBox> ApplyResetCancelBox;

	/** ListView container Box */
	TSharedPtr<SBox> RenamePreviewListBox;

	/** ListView Header row */
	TSharedPtr<SHeaderRow> RenamePreviewListHeaderRow;

	/** ListView of the Previews */
	TSharedPtr<SListView<TSharedPtr<FAdvancedRenamerPreview>>> RenamePreviewList;
};
