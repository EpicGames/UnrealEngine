// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Layout/Margin.h"
#include "UObject/NameTypes.h"

enum class EAdvancedRenamerRemoveOldType : uint8
{
	Separator,
	Chars
};

namespace AdvancedRenamerSlateUtils::Default
{
	static FName OriginalNameColumnName = TEXT("OriginalName");
	static FName NewNameColumnName = TEXT("NewName");
	static FMargin FirstWidgetPadding = FMargin(0.f, 0.f, 4.f, 0.f);
	static FMargin LastWidgetPadding = FMargin(4.f, 0.f, 0.f, 0.f);
	static FMargin MiddleWidgetPadding = FMargin(4.f, 0.f, 4.f, 0.f);
	static FMargin SectionContentFirstEntryPadding = FMargin(8.f, 8.f);
	static FMargin SectionContentMiddleEntriesPadding = FMargin(8.f, 0.f, 8.f, 8.f);
	static FMargin VerticalPadding = FMargin(0.f, 4.f);
	static FMargin ChangeCaseFirstButtonPadding = FMargin(0.f, 0.f, 4.f, 0.f);
	static FMargin ChangeCaseMiddleButtonsPadding = FMargin(4.f, 0.f);
	static FMargin ChangeCaseLastButtonPadding = FMargin(4.f, 0.f, 0.f, 0.f);
	static FMargin ApplyButtonPadding = FMargin(8.f, 8.f, 4.f, 8.f);
	static FMargin ResetButtonPadding = FMargin(4.f, 8.f, 4.f, 8.f);
	static FMargin CancelButtonPadding = FMargin(4.f, 8.f, 8.f, 8.f);
}
