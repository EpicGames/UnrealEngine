// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointerFwd.h"

class AActor;
class IAdvancedRenamer;
class IAdvancedRenamerProvider;
class IAdvancedRenamerSection;
class IToolkitHost;
class SWidget;

/** Advanced Rename Panel Plugin - Easily bulk rename stuff! */
class IAdvancedRenamerModule : public IModuleInterface
{
protected:
	static constexpr const TCHAR* ModuleName = TEXT("AdvancedRenamer");

public:
	static bool IsLoaded()
	{
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}

	static IAdvancedRenamerModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IAdvancedRenamerModule>(ModuleName);
	}

	virtual TSharedRef<IAdvancedRenamer> CreateAdvancedRenamer(const TSharedRef<IAdvancedRenamerProvider>& InRenameProvider) = 0;

	virtual void OpenAdvancedRenamer(const TSharedRef<IAdvancedRenamerProvider>& InRenameProvider, const TSharedPtr<IToolkitHost>& InToolkitHost) = 0;

	virtual void OpenAdvancedRenamer(const TSharedRef<IAdvancedRenamerProvider>& InRenameProvider, const TSharedPtr<SWidget>& InParentWidget) = 0;

	virtual void OpenAdvancedRenamerForActors(const TArray<AActor*>& InActors, const TSharedPtr<IToolkitHost>& InToolkitHost) = 0;

	virtual void OpenAdvancedRenamerForActors(const TArray<AActor*>& InActors, const TSharedPtr<SWidget>& InParentWidget) = 0;

	virtual void OpenAdvancedRenamer(const TSharedRef<IAdvancedRenamer>& InRenamer, const TSharedPtr<IToolkitHost>& InToolkitHost) = 0;

	virtual void OpenAdvancedRenamer(const TSharedRef<IAdvancedRenamer>& InRenamer, const TSharedPtr<SWidget>& InParentWidget) = 0;

	virtual TArray<TSharedPtr<IAdvancedRenamerSection>> GetRegisteredSections() const = 0;

	virtual TArray<AActor*> GetActorsSharingClassesInWorld(const TArray<AActor*>& InActors) = 0;

	// Delegate to filter out actors before they are opened in the advanced renamer. The renamer will not be opened if the array is empty
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnFilterAdvancedRenamerActors, TArray<TWeakObjectPtr<AActor>>&);
	virtual FOnFilterAdvancedRenamerActors& OnFilterAdvancedRenamerActors() = 0;
	
};
