// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/CoreMiscDefines.h"

class FString;

class IAdvancedRenamerProvider
{
public:
	virtual ~IAdvancedRenamerProvider() = default;
	virtual int32 Num() const = 0;
	virtual bool IsValidIndex(int32 InIndex) const = 0;
	virtual uint32 GetHash(int32 InIndex) const = 0;
	virtual FString GetOriginalName(int32 InIndex) const = 0;
	virtual bool RemoveIndex(int32 InIndex) = 0;
	virtual bool CanRename(int32 InIndex) const = 0;

	virtual bool BeginRename() = 0;
	virtual bool PrepareRename(int32 InIndex, const FString& InNewName) = 0;
	virtual bool ExecuteRename() = 0;
	virtual bool EndRename() = 0;

	virtual int32 FindHash(int32 InInHash) const
	{
		const int32 Count = Num();

		for (int32 Index = 0; Index < Count; ++Index)
		{
			if (GetHash(Index) == InInHash)
			{
				return Index;
			}
		}

		return INDEX_NONE;
	}
};
